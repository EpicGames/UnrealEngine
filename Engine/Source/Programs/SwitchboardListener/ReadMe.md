# Usage

The listener requires the command line switches -ip and -port to be set.
Example: SwitchboardListener -ip=127.0.0.1 -port=2980

# Protocol

All commands/responses between SB and its listeners are json encoded.
Every command has an ID (filled in by Switchboard) so it is possible to match the responses sent by the listener.

## Switchboard to Listener

List of commands sent from Switchboard to the Listener:
- Start process/Run executable
{
	'command' : 'start',
	'id' : '16fd2706-8baf-433b-82eb-8c7fada847da',
	'exe' : 'notepad.exe',
	'args': '--help'
}
- Kill previously started process
{
	'command' : 'kill',
	'id' : '16fd2706-8baf-433b-82eb-8c7fada847da',
	'program id' : '6fa459ea-ee8a-3ca4-894e-db77e160355e'
}
- Killall processes that were started through the listener
{
	'command' : 'killall',
	'id': '16fd2706-8baf-433b-82eb-8c7fada847da'
}
- Send file
{
	'command': 'send file',
	'id': '16fd2706-8baf-433b-82eb-8c7fada847da',
	# the destination accepts the variables %TEMP% and %RANDOM%
	# %TEMP% will be replaced with a path to the system's temp folder
	# %RANDOM% will generate a random string and make sure that the resulting path does not exist yet
	# example: %TEMP%/folder/%RANDOM%.jpg would result in something like C:\Users\Username\AppData\Local\Temp\folder\718938484A70E535DBC5648DEACEBFE5.jpg
	'destination': 'absolute system path to destination file',
	'content': base64 encoded file content
}
- Receive file
{
	'command': 'receive file',
	'id': '16fd2706-8baf-433b-82eb-8c7fada847da',
	'source': 'absolute path to file on listener machine'
}
- Disconnect, lets the listener know the client is about to disconnect from the socket
{
	'command': 'disconnect',
	'id': '16fd2706-8baf-433b-82eb-8c7fada847da'
}
- KeepAlive, avoid getting disconnected by the listener. this command will not cause a response from the server.
{
	'command': 'keep alive',
	'id': '16fd2706-8baf-433b-82eb-8c7fada847da'
}

## Listener to Switchboard

Responses sent by the listener to Switchboard:
- Command Accepted
{
	'command accepted': True,
	'id': '16fd2706-8baf-433b-82eb-8c7fada847da'
}
- Command Declined
{
	'command accepted': False,
	'id': '16fd2706-8baf-433b-82eb-8c7fada847da',
	'error': string
}
- Program Started
{
	'program started': True,
	'program id': '6fa459ea-ee8a-3ca4-894e-db77e160355e',
	'message id': '16fd2706-8baf-433b-82eb-8c7fada847da'
}
- Program start failed
{
	'program started': False,
	'error': string,
	'message id': '16fd2706-8baf-433b-82eb-8c7fada847da'
}
- Program ended (on its own)
{
	'program ended': True,
	'program id' : '6fa459ea-ee8a-3ca4-894e-db77e160355e',
	'returncode': ret,
	'output': string
}
- Program was killed
{
	'program killed': True,
	'program id': '6fa459ea-ee8a-3ca4-894e-db77e160355e'
}
- Killing program failed
{
	'program killed': False,
	'program id': '6fa459ea-ee8a-3ca4-894e-db77e160355e',
	'error': string
}
- Send file completed
{
	'send file complete': True,
	# the path to the file that was actually written to, which might be different from the path specified
	# in the send command in case variables had been specified
	'destination': string
}
- Send file failed
{
	'send file complete': false,
	'destination': string,
	'error': string
}
- Receive file completed
{
	'receive file complete': True,
	'content': 'base64 encoded file content'
}
- Receive file failed
{
	'receive file complete': False,
	'source': 'path to source file that listener tried to send',
	'error': string
}
