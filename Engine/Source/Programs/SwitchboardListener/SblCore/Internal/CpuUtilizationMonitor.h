// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Logging/LogMacros.h"


DECLARE_LOG_CATEGORY_EXTERN(LogCpuUtilizationMonitor, Log, All);


// Default implementation does nothing (functionality optional and only implemented for some platforms).
// Instead, use the FCpuUtilizationMonitor typedef to get a platform-specific implementation if available.
class FGenericCpuUtilizationMonitor
{
public:
	virtual ~FGenericCpuUtilizationMonitor()
	{
	}

	virtual bool IsInitialized() const
	{
		return false;
	}

	/**
	 * Samples the current CPU utilization, provided as per-core values ranging from 0-100.
	 *
	 * @param OutCoreUtilization The array that will receive the sampled values.
	 * @return True if OutCoreUtilization is valid (new or recently cached values were available).
	 */
	virtual bool GetPerCoreUtilization(TArray<int8>& OutCoreUtilization)
	{
		return false;
	}
};


#if PLATFORM_WINDOWS
#include COMPILED_PLATFORM_HEADER(CpuUtilizationMonitor.h)
#else
typedef FGenericCpuUtilizationMonitor FCpuUtilizationMonitor;
#endif
