// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PosixOS/PosixOSSwitchboardCredential.h"


namespace UE::SwitchboardListener
{
	inline ICredentialManager& ICredentialManager::GetPlatformCredentialManager()
	{
		using namespace UE::SwitchboardListener::Private;
		static FPosixCredentialManager CredentialManager;
		return CredentialManager;
	}
};
