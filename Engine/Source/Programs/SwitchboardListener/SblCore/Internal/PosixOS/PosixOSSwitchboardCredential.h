// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SwitchboardCredentialInterface.h"


namespace UE::SwitchboardListener::Private
{
	class FPosixCredentialManager : public ICredentialManager
	{
	public:
		virtual TSharedPtr<ICredential> LoadCredential(FStringView InCredentialName) override;
		virtual bool SaveCredential(FStringView InCredentialName, FStringView InUser, FStringView InBlob) override;
	};
};
