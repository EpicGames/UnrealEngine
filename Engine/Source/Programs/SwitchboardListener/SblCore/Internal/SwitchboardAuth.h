// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Containers/Utf8String.h"
#include "Misc/CoreMiscDefines.h" // UE_PUSH_MACRO
#include "Misc/DateTime.h"
#include "Misc/TVariant.h"

UE_PUSH_MACRO("UI")
// Workaround for "ossl_typ.h(144): error C2365: 'UI': redefinition; previous definition was 'namespace'"
//                "ObjectMacros.h(872): note: see declaration of 'UI'"
#define UI UI_ST
#include <openssl/ossl_typ.h>
UE_POP_MACRO("UI")


class FJsonWebToken;

namespace UE::SwitchboardListener
{
	class ICredentialManager;
}


namespace UE::SwitchboardListener
{
	/**
	 * Get the default paths for self-signed key pairs,
	 * or (on some platforms) persisted credentials.
	 * 
	 * - Windows: %LocalAppData%\SwitchboardListener
	 * - PosixOS family: ~/.switchboard/listener
	 */
	FString GetSwitchboardDataDirectory();

	/**
	 * Disable console echo and prompt the user to enter a password (twice).
	 * 
	 * @param bAllowEmpty Whether to allow the user to leave the input empty.
	 * 
	 * @return The password the user entered.
	 */
	FString ReadPasswordFromStdin(bool bAllowEmpty = false);

	/**
	 * Completely overwrites the contents of the provided array with
	 * cryptographically secure pseudo-random bytes.
	 * 
	 * @param InOutArray Set to the desired length before calling.
	 */
	void FillSecureRandom(TArrayView<uint8> InOutArray);
};


/**
 * - Uses either the specified key pair, or an automatically generated
 *   self-signed key pair. On systems where credentials can be stored
 *   encrypted at rest, the resulting private key is protected with a
 *   randomly generated password.
 *
 * - Can sign and verify JSON Web Tokens using the loaded key pair.
 * 
 * - Can generate, persist, and constant-time compare password hashes using
 *   scrypt key derivation.
 * 
 * - Has helper methods for MsQuic interop.
 */
class FSwitchboardAuthHelper
{
public:
	struct FSettings
	{
		TOptional<TVariant<FString, X509*>> Certificate;
		TOptional<TVariant<FString, EVP_PKEY*>> PrivateKey;
		TOptional<FUtf8StringView> PrivateKeyPassword;

		UE::SwitchboardListener::ICredentialManager* CredentialManager = nullptr;

		FSettings()
		{
			// Without this (or with `= default`), clang gives the following:
			// `error: default member initializer for 'CredentialManager' needed within definition
			// of enclosing class 'FSwitchboardAuthHelper' outside of member functions`
		}
	};

	bool Initialize(const FSettings& InSettings = FSettings());
	void Shutdown();


	/** May be empty if the certificate was not loaded from file. */
	const FUtf8String& GetCertificateFilePath() const;

	/** May be empty if the private key was not loaded from file. */
	const FUtf8String& GetPrivateKeyFilePath() const;

	/** May be null if the private key was not password-protected. */
	const UTF8CHAR* GetPrivateKeyPassword() const;


	/** Change the required password. */
	bool SetAuthPassword(FStringView InNewPassword, bool bExpireExistingTokens = true);

	/** Validate whether the provided password attempt matches the expected password. */
	bool ValidatePassword(FStringView InAttempt);

	/** Retrieves the plain text; only supported on platforms where the credential can be encrypted at rest. */
	const UTF8CHAR* GetAuthPassword() const;

	/** True if a password hash is set, even if `GetAuthPassword` would return unset. */
	bool IsAuthPasswordSet() const;


	/** Returns a JWT, signed with our private key, containing the provided payload. */
	FString IssueJWT(const TMap<FStringView, FStringView>& InPayloadFields);

	/** Returns the decoded and validated JWT; result is unset if validation fails. */
	TOptional<FJsonWebToken> GetValidatedJWT(FStringView InEncodedJwt) const;

	/** Returns just whether the JWT passes validation. */
	bool IsValidJWT(FStringView InEncodedJwt) const;


private:
	bool CheckOrCreateSelfSignedKeyPair(FSettings& InOutSettings);
	FUtf8String GenerateHashSalt();
	bool UpdateLastPasswordChangeTimeUtc(FDateTime InChangeTimeUtc = FDateTime::UtcNow());

private:
	UE::SwitchboardListener::ICredentialManager* CredentialManager;
	bool bSecureHeapNeedsShutdown = false;

	FUtf8String CertificateFilePath;
	FUtf8String PrivateKeyFilePath;
	UTF8CHAR* PrivateKeyPassword = nullptr;
	EVP_PKEY* PrivateKey = nullptr;
	X509* Certificate = nullptr;

	UTF8CHAR* PlainAuthPassword = nullptr;
	FUtf8String HashSalt;
	TArray<uint8> HashedAuthPassword;
	int64 LastPasswordChangeTimeUtc = -1;
};


namespace UE::SwitchboardListener::Private
{
#if !PLATFORM_WINDOWS
	static constexpr mode_t CredentialFilePermissionFlags = S_IRUSR | S_IWUSR;
	static constexpr mode_t CredentialDirPermissionFlags = S_IRUSR | S_IWUSR | S_IXUSR;
	static constexpr mode_t CredentialPermissionMask = S_IRWXU | S_IRWXG | S_IRWXO;
#endif

	bool CheckCredentialPermissions(FStringView InFilePath);
	TPair<EVP_PKEY*, X509*> CreateSelfSignedPair();
};
