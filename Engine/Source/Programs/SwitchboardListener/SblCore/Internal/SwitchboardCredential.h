// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include COMPILED_PLATFORM_HEADER(SwitchboardCredential.h)
