// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "HAL/PreprocessorHelpers.h"
#include "Templates/SharedPointer.h"


namespace UE::SwitchboardListener
{
	class ICredential : public TSharedFromThis<ICredential>
	{
	public:
		virtual ~ICredential() = default;

		/** Returns the user field value for this credential. */
		virtual FStringView GetUser() = 0;

		/** Returns the blob field value (password, token, etc) for this credential. */
		virtual FStringView GetBlob() = 0;
	};

	class ICredentialManager : public TSharedFromThis<ICredentialManager>
	{
	public:
		/** Returns the default credential manager for this platform. */
		static ICredentialManager& GetPlatformCredentialManager();

	public:
		virtual ~ICredentialManager() = default;

		/** Returns whether credentials are encrypted at rest on the current platform. */
		virtual bool IsEncryptedAtRest() const { return false; }

		/** Returns the stored credential if it exists. */
		virtual TSharedPtr<ICredential> LoadCredential(FStringView InCredentialName) = 0;

		/** Stores the provided credential. */
		virtual bool SaveCredential(FStringView InCredentialName, FStringView InUser, FStringView InBlob) = 0;
	};
};
