// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SwitchboardListenerApp.h"
#include "SwitchboardAuth.h"
#include "Async/Mutex.h"
#include "Async/RecursiveMutex.h"
#include "Containers/BitArray.h"
#include "Containers/Queue.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"

#include "CpuUtilizationMonitor.h"


#if !PLATFORM_WINDOWS
#	include <msquic.h>
#else
#	include "Windows/AllowWindowsPlatformTypes.h"
#	include <msquic.h>
#	include "Windows/HideWindowsPlatformTypes.h"
#endif


struct FRunningProcess;
struct FSwitchboardMessageFuture;
struct FSwitchboardTask;
struct FSwitchboardAuthenticateTask;
struct FSwitchboardDisconnectTask;
struct FSwitchboardSendFileToClientTask;
struct FSwitchboardStartTask;
struct FSwitchboardKillTask;
struct FSwitchboardReceiveFileFromClientTask;
struct FSwitchboardGetSyncStatusTask;
struct FSwitchboardRefreshMosaicsTask;
struct FSwitchboardRedeployListenerTask;
struct FSwitchboardFixExeFlagsTask;
struct FSwitchboardMinimizeWindowsTask;
struct FSwitchboardSetInactiveTimeoutTask;
struct FSwitchboardFreeListenerBinaryTask;
struct FSwitchboardStartLiveLogStreamTask;
struct FSwitchboardStopLiveLogStreamTask;

class FGpuClockerManager;


struct FSwitchboardCommandLineOptions
{
	bool OutputVersion = false;
	bool MinimizeOnLaunch = true;

	TOptional<FIPv4Address> Address;
	TOptional<uint16> Port;

	TOptional<uint32> RedeployFromPid;

	static FSwitchboardCommandLineOptions FromString(const TCHAR* CommandLine);
	FString ToString(bool bIncludeRedeploy = false) const;
};


struct FRedeployStatus
{
	enum class EState : uint8
	{
		// No redeploy initiated.
		None,

		// Redeploy initiated by client, decline subsequent redeploy requests.
		RequestReceived,

		// New listener executable hash verified and written to randomly-named temporary file.
		// This also means we've stopped listening for new connections in this (old) listener.
		NewListenerWrittenTemp,

		// New listener successfully launched and running.
		NewListenerStarted,

		// Current process executable moved to randomly-named temporary file.
		ThisListenerRenamed,

		// New listener executable moved to original location of this executable.
		NewListenerRenamed,

		// Child process has signaled via IPC that it's initialized; shutting down.
		Complete,
	};

	bool InProgress() const
	{
		return State != EState::None && State != EState::Complete;
	}

	EState State = EState::None;

	// Valid when State >= RequestReceived
	FIPv4Endpoint RequestingClient;

	// Valid when State >= NewListenerWrittenTemp
	FString OriginalThisExePath;
	FString TempNewListenerPath;

	// Valid when State >= NewListenerStarted
	FProcHandle ListenerProc;

	// Valid when State >= ThisListenerRenamed
	FString TempThisListenerPath;
};


class FSwitchboardListener
{
	struct FConnection;
	using FConnectionRef = TSharedRef<FConnection>;
	using FByteArrayRef = TSharedRef<TArray<uint8>>;

	static const FIPv4Endpoint InvalidEndpoint;

public:
	explicit FSwitchboardListener(const FSwitchboardCommandLineOptions& InOptions);
	~FSwitchboardListener();

	bool Init();
	void Shutdown();

	void Tick();

	bool StartListening();
	bool StopListening();
	bool IsListening() const { return QuicApi != nullptr; }

	/** May be null if the plain text password cannot be stored encrypted at rest on this platform. */
	const UTF8CHAR* GetAuthPassword() const;
	bool IsAuthPasswordSet() const;
	bool SetAuthPassword(const FString& NewPassword);

	TSet<FIPv4Endpoint> GetConnectedClientEndpoints() const;
	TSet<FIPv4Address> GetConnectedClientAddresses() const;

	// Used for redeploy synchronization.
	static FString GetIpcSemaphoreName(uint32 ParentPid);

	FSimpleMulticastDelegate& OnInit() { return OnInitDelegate; }
	FSimpleMulticastDelegate& OnShutdown() { return OnShutdownDelegate; }
	FSimpleMulticastDelegate& OnTick() { return OnTickDelegate; }

private:
	bool ParseIncomingMessage(const FString& InMessage, const FIPv4Endpoint& InEndpoint, const FConnectionRef& Connection);

	bool RunScheduledTask(const FSwitchboardTask& InTask);
	bool Task_Authenticate(const FSwitchboardAuthenticateTask& InAuthTask);
	bool Task_StartProcess(const FSwitchboardStartTask& InRunTask);
	bool Task_KillProcess(const FSwitchboardKillTask& KillTask);
	bool Task_ReceiveFileFromClient(const FSwitchboardReceiveFileFromClientTask& InReceiveFileFromClientTask);
	bool Task_RedeployListener(const FSwitchboardRedeployListenerTask& InRedeployListenerTask);
	bool Task_SendFileToClient(const FSwitchboardSendFileToClientTask& InSendFileToClientTask);
	bool Task_GetSyncStatus(const FSwitchboardGetSyncStatusTask& InGetSyncStatusTask);
	bool Task_RefreshMosaics(const FSwitchboardRefreshMosaicsTask& InRefreshMosaicsTask);
	bool Task_FixExeFlags(const FSwitchboardFixExeFlagsTask& InFixExeFlagsTask);
	bool Task_MinimizeWindows(const FSwitchboardMinimizeWindowsTask& InRefreshMosaicsTask);
	bool Task_SetInactiveTimeout(const FSwitchboardSetInactiveTimeoutTask& InTimeoutTask);
	bool Task_FreeListenerBinary(const FSwitchboardFreeListenerBinaryTask& InRenameProcessTask);

	/** Starts streaming a log file to Switchboard */
	bool Task_StartLiveLogStream(const FSwitchboardStartLiveLogStreamTask& InStartLiveLogStreamTask);

	/** Stops streaming a log file to Switchboard */
	bool Task_StopLiveLogStream(const FSwitchboardStopLiveLogStreamTask& InStopLiveLogStreamTask);

	bool KillProcessNow(FRunningProcess* InProcess, float SoftKillTimeout = 0.0f);

#if PLATFORM_WINDOWS
	using FFlipModeMonitorSessionId = uint8;
	static constexpr FFlipModeMonitorSessionId InvalidFlipModeSessionId = FFlipModeMonitorSessionId(-1);
	static constexpr FFlipModeMonitorSessionId MaxFlipModeMonitorSessions = 16;
	void HarvestExistingFlipModeSessions();
	bool EnsureFlipModeSessionAvailable(FFlipModeMonitorSessionId SessionId, bool* bOutStoppedSession = nullptr);
	FFlipModeMonitorSessionId AllocFlipModeMonitorSessionId();
	void FreeFlipModeMonitorSessionId(FFlipModeMonitorSessionId SessionId);
	FString MakeFlipModeMonitorSessionName(FFlipModeMonitorSessionId SessionId);
	FRunningProcess* FindOrStartFlipModeMonitorForUUID(const FGuid& UUID);
#endif

	void CleanUpDisconnectedSockets();
	void HandleStdout(const TSharedPtr<FRunningProcess>& Process);
	void HandleRunningProcesses(TArray<TSharedPtr<FRunningProcess>>& Processes, bool bNotifyThatProgramEnded);

	/** Processes all active live log streams, reading new content and sending to Switchboard clients */
	void HandleLiveLogStreams();

	bool SendMessage(const FString& InMessage, const FIPv4Endpoint& InEndpoint);
	void SendMessageFutures();

	bool EquivalentTaskFutureExists(uint32 TaskEquivalenceHash) const;

	void RollbackRedeploy();

	void FillStatePacket(struct FSwitchboardStatePacket& OutStatePacket);

	_Function_class_(QUIC_LISTENER_CALLBACK)
	static QUIC_STATUS QUIC_API QuicListenerThunk(HQUIC Listener, void* Context, QUIC_LISTENER_EVENT* Event);
	static QUIC_STATUS QUIC_API QuicConnectionThunk(HQUIC Connection, void* Context, QUIC_CONNECTION_EVENT* Event);
	static QUIC_STATUS QUIC_API QuicStreamThunk(HQUIC Stream, void* Context, QUIC_STREAM_EVENT* Event);
	QUIC_STATUS QuicListenerCallback(HQUIC Listener, QUIC_LISTENER_EVENT* Event);
	QUIC_STATUS QuicConnectionCallback(HQUIC Connection, QUIC_CONNECTION_EVENT* Event);
	QUIC_STATUS QuicStreamCallback(HQUIC Stream, QUIC_STREAM_EVENT* Event);

private: // Private nested types.
	/** Combines a QUIC_BUFFER span (ptr + len) and its backing array. */
	struct FQuicBuffer
	{
		QUIC_BUFFER QuicBuffer;
		FByteArrayRef Storage;

		FQuicBuffer()
			: QuicBuffer{ .Length = 0, .Buffer = nullptr }
			, Storage(MakeShared<TArray<uint8>>())
		{
		}

		explicit FQuicBuffer(FByteArrayRef&& InStorage)
			: Storage(MoveTemp(InStorage))
		{
			QuicBuffer = {
				.Length = static_cast<uint32>(Storage->Num()),
				.Buffer = Storage->GetData(),
			};
		}
	};

	/** Tracks all the state for an established connection. */
	struct FConnection
	{
		FIPv4Endpoint Endpoint;
		HQUIC QuicConn = nullptr;
		// We currently assume a single bidirectional stream initiated by the peer.
		HQUIC QuicStream = nullptr;

		UE::FMutex SendLock;
		TQueue<TSharedPtr<FQuicBuffer>> SendBuffers;

		UE::FMutex ReceiveLock;
		bool bMessageComplete = false;
		FByteArrayRef ReceiveBuffer = MakeShared<TArray<uint8>>();

		bool bAuthenticated = false;

		// TODO?: MsQuic handles idle timeout, but configuration is app-wide.
		// If we want to support the existing per-client override, need to
		// figure out how best to pass one override when calling
		// QuicApi->ConnectionSetConfiguration() (clone config obj N times?).
		float InactiveTimeout;
		double LastActivityTime;
	};

private:
	FSwitchboardCommandLineOptions Options;
	TUniquePtr<FIPv4Endpoint> ListenerEndpoint;

	FSwitchboardAuthHelper AuthHelper;

	/** MsQuic top level function table for all other API calls. */
	const QUIC_API_TABLE* QuicApi = nullptr;

	/** MsQuic registration manages the execution context for all child objects. */
	HQUIC QuicRegistration = nullptr;

	/** MsQuic configuration manages security-related and other common QUIC settings. */
	HQUIC QuicConfiguration = nullptr;

	/** MsQuic incoming connection handler. */
	HQUIC QuicListener = nullptr;

	UE::FRecursiveMutex ConnectionsLock;
	TMap<FIPv4Endpoint, FConnectionRef> ConnectionsByEndpoint;
	TMap<HQUIC, FConnectionRef> ConnectionsByQuicConn;
	TMap<HQUIC, FConnectionRef> ConnectionsByQuicStream;

	static constexpr int8 MaxAuthFailures = 5;
	TMap<FIPv4Address, int8> AuthFailuresByAddress;

	TQueue<TUniquePtr<FSwitchboardTask>, EQueueMode::Spsc> ScheduledTasks;
	TQueue<TUniquePtr<FSwitchboardTask>, EQueueMode::Spsc> DisconnectTasks;
	TArray<TSharedPtr<FRunningProcess>> RunningProcesses;
	TArray<TSharedPtr<FRunningProcess>> FlipModeMonitors;

#if PLATFORM_WINDOWS
	TBitArray<> RunningFlipModeSessions;
#endif

	TArray<FSwitchboardMessageFuture> MessagesFutures;
	TSharedPtr<FCpuUtilizationMonitor> CpuMonitor;

	/** Gpu clock manager. Used to lock/unlock gpu clocks. */
	TSharedPtr<FGpuClockerManager> GpuClockerManager;

	bool bProcessorSMT;

	TSharedPtr<FRWLock> CachedMosaicToposLock;
	TSharedPtr<TArray<struct FMosaicTopo>> CachedMosaicTopos;

	FRedeployStatus RedeployStatus;

	FSimpleMulticastDelegate OnInitDelegate;
	FSimpleMulticastDelegate OnShutdownDelegate;
	FSimpleMulticastDelegate OnTickDelegate;

	/** Live log streaming state */
	struct FLiveLogStream
	{
		/** Local path to the log file being streamed */
		FString LogFilePath;

		/** Network endpoint of the client receiving the stream */
		FIPv4Endpoint ClientEndpoint;

		/** Current read position in the log file */
		int64 LastFilePosition;

		/** If true, only stream new content added after the stream is started */
		bool bTailOnly;

		/** Timestamp of the last stream update check */
		double LastUpdateTime;

		/** UUID of the process whose log is being streamed */
		FGuid AssociatedProcessUUID;

		FLiveLogStream(const FString& InLogFilePath, const FIPv4Endpoint& InClientEndpoint, bool bInTailOnly, const FGuid& InProcessUUID = FGuid(), int64 InStartOffset = -1)
			: LogFilePath(InLogFilePath)
			, ClientEndpoint(InClientEndpoint)
			, LastFilePosition(InStartOffset >= 0 ? InStartOffset : 0)
			, bTailOnly(bInTailOnly)
			, LastUpdateTime(0.0)
			, AssociatedProcessUUID(InProcessUUID)
		{}
	};

	TArray<FLiveLogStream> ActiveLogStreams;
	UE::FMutex LiveLogStreamsMutex;
};
