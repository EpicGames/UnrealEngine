// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"


DECLARE_LOG_CATEGORY_EXTERN(LogSwitchboard, Verbose, All);
DECLARE_LOG_CATEGORY_EXTERN(LogSwitchboardProtocol, Verbose, All);


#if !defined(SBL_SLATE)
#	define SBL_SLATE 0
#endif


#if !defined(SWITCHBOARD_CPUPROFILERTRACE_ENABLED)
#	if CPUPROFILERTRACE_ENABLED
#		define SWITCHBOARD_CPUPROFILERTRACE_ENABLED 1
#	else
#		define SWITCHBOARD_CPUPROFILERTRACE_ENABLED 0
#	endif
#endif


#if SWITCHBOARD_CPUPROFILERTRACE_ENABLED
#	include "ProfilingDebugging/CpuProfilerTrace.h"

#	define SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(Name) \
		TRACE_CPUPROFILER_EVENT_SCOPE(Name)

#	define SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE_STR(Name) \
		TRACE_CPUPROFILER_EVENT_SCOPE_STR(Name)

#	define SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(Name) \
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(Name)

#else

#	define SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(Name)
#	define SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE_STR(Name)
#	define SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(Name)

#endif
