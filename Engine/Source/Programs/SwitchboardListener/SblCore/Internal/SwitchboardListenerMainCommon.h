// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"


struct FSwitchboardCommandLineOptions;

bool SwitchboardListenerMainInit(const FSwitchboardCommandLineOptions& InOptions);
bool SwitchboardListenerMainShutdown();

int32 SwitchboardListenerMainWrapper();
