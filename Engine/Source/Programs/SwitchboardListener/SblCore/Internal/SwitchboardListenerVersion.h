// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define SBLISTENER_VERSION_MAJOR 3
#define SBLISTENER_VERSION_MINOR 5
#define SBLISTENER_VERSION_PATCH 0
