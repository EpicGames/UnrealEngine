// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Optional.h"


/**
 * Alternative implementation of FPlatformProcess::CreateProc with some additional features.
 * Currently, the primary difference is that handles must be explicitly marked for inheritance.
 */
struct FGenericSwitchboardProcess
{
public:
	struct FCreateProcParams
	{
		/** Path to the executable to launch. */
		FStringView Executable;

		/** Command line arguments. */
		TArrayView<const FStringView> Args;

		/** If true, the new process will have its own window. */
		bool bLaunchDetached = false;

		/** If true, the new process will be minimized in the task bar. */
		bool bLaunchMinimized = false;

		/** If true, the new process will not have a window or be in the task bar. */
		bool bLaunchReallyHidden = false;

		/** -2 idle, -1 low, 0 normal, 1 high, 2 higher */
		int32 PriorityModifier = 0;

		/** Working directory in which to start the program. If unset, uses the current working directory. */
		TOptional<FStringView> WorkingDirectory;

		/** The read end of a pipe which will serve as stdin for the child process. */
		void* StdinPipe = nullptr;

		/** The write end of a pipe which will serve as stdout for the child process. */
		void* StdoutPipe = nullptr;

		/** The write end of a pipe which will serve as stderr for the child process. */
		void* StderrPipe = nullptr;

		/**
		 * Additional handles which the descendant process is permitted to inherit.
		 * StdinPipe, StdoutPipe, and StderrPipe are always inheritable, and should not be added to this list.
		 * Ignored if `bInheritAllHandles == true` (legacy behavior; not recommended).
		 */
		TArrayView<void*> AdditionalInheritedHandles;

		/**
		 * Permit child process to inherit any handles from the parent process not explicitly excluded otherwise.
		 * This was the legacy behavior if any standard I/O redirection pipes were specified.
		 * This can lead to arcane resource leaks, and is not recommended.
		 */
		bool bInheritAllHandles = false;
	};

	struct FCreateProcResult
	{
		FProcHandle Handle;
		uint32 ProcessId = 0;
	};

public:
	static FCreateProcResult CreateProc(const FCreateProcParams& InParams);

	// FPlatformProcess-compatible signature that calls into our implementation with more restrictive handle inheritance.
	static FProcHandle CreateProc(
		const TCHAR* URL,
		const TCHAR* Parms,
		bool bLaunchDetached,
		bool bLaunchHidden,
		bool bLaunchReallyHidden,
		uint32* OutProcessID,
		int32 PriorityModifier,
		const TCHAR* OptionalWorkingDirectory,
		void* PipeWriteChild,
		void* PipeReadChild = nullptr
	);

	// FPlatformProcess-compatible signature that calls into our implementation with more restrictive handle inheritance.
	static FProcHandle CreateProc(
		const TCHAR* URL,
		const TCHAR* Parms,
		bool bLaunchDetached,
		bool bLaunchHidden,
		bool bLaunchReallyHidden,
		uint32* OutProcessID,
		int32 PriorityModifier,
		const TCHAR* OptionalWorkingDirectory,
		void* PipeWriteChild,
		void* PipeReadChild,
		void* PipeStdErrChild
	);
};


// TODO: Restricted FD inheritance is achievable with POSIX_SPAWN_CLOEXEC_DEFAULT,
// and maybe posix_spawn_file_actions_addclosefrom_np?
#if PLATFORM_WINDOWS
#include COMPILED_PLATFORM_HEADER(SwitchboardProcess.h)
#else
using FSwitchboardProcess = FGenericSwitchboardProcess;
#endif
