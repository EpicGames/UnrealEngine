// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef _WIN32_WINDOWS
#define _WIN32_WINDOWS 0x0410
#endif

#include "CpuUtilizationMonitor.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <Windows.h>
#include <Pdh.h>
#include <PdhMsg.h>
#include "Windows/HideWindowsPlatformTypes.h"


class FWindowsCpuUtilizationMonitor : public FGenericCpuUtilizationMonitor
{
public:
	FWindowsCpuUtilizationMonitor();
	~FWindowsCpuUtilizationMonitor();

	bool IsInitialized() const override
	{
		return bIsInitialized;
	}

	bool GetPerCoreUtilization(TArray<int8>& OutCoreUtilization) override;

private:
	bool QueryUpdatedUtilization();

	bool bIsInitialized = false;

	HQUERY QueryHandle = INVALID_HANDLE_VALUE;
	TArray<PDH_HCOUNTER> CounterHandles;
	double LastQueryTime = DBL_MIN;
	TArray<int8> CachedCoreUtilization;
};

typedef FWindowsCpuUtilizationMonitor FCpuUtilizationMonitor;
