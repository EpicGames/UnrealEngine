// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SwitchboardCredentialInterface.h"


namespace UE::SwitchboardListener::Private
{
	class FWindowsCredentialManager : public ICredentialManager
	{
	public:
		virtual bool IsEncryptedAtRest() const override { return true; }
		virtual TSharedPtr<ICredential> LoadCredential(FStringView InCredentialName) override;
		virtual bool SaveCredential(FStringView InCredentialName, FStringView InUser, FStringView InBlob) override;
	};
};


namespace UE::SwitchboardListener
{
	inline ICredentialManager& ICredentialManager::GetPlatformCredentialManager()
	{
		using namespace UE::SwitchboardListener::Private;
		static FWindowsCredentialManager CredentialManager;
		return CredentialManager;
	}
};
