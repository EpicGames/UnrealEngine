// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SwitchboardProcess.h"


/** Windows implementation of FSwitchboardProcess. See FGenericSwitchboardProcess for details. */
struct FWindowsSwitchboardProcess : public FGenericSwitchboardProcess
{
public:
	using FGenericSwitchboardProcess::CreateProc;

	static FCreateProcResult CreateProc(const FCreateProcParams& InParams);
};


using FSwitchboardProcess = FWindowsSwitchboardProcess;
