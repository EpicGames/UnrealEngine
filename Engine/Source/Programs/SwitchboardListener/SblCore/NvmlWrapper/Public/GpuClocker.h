// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <stdint.h>

#include "NvmlWrapperPublic.h"

/** Encapsulates the logic to lock/unlock gpu clocks */
class EXPORTLIB FGpuClocker
{
public:

	/** Locks the gpu clocks to the maximum allowed */
	bool LockGpuClocks() const;

	/** Unlocks the gpu and memory clocks */
	bool UnlockGpuClocks() const;

	/** Locks the given gpu and memory clocks to the maximum allowed */
	bool GpuUnlockDevice(const uint32_t GpuIdx) const;

	/** Unlocks the given gpu and memory clocks */
	bool GpuLockDevice(const uint32_t GpuIdx) const;

	/** Returns number of GPUs */
	bool GetGpuCount(uint32_t& OutNumGpus) const;

	/** Returns the temperature in Celsius of the indexed GPU */
	bool GetGpuCelsius(const uint32_t GpuIdx, uint32_t& OutCelsius) const;

	/** Returns the GPU usage in percentage of running kernels and memory operations */
	bool GetGpuUsage(const uint32_t GpuIdx, uint32_t& OutUsageKernel, uint32_t& OutUsageMemory) const;

	/** Return GPU clock frequency in MHz */
	bool GetGpuMHz(const uint32_t GpuIdx, uint32_t& GraphicsMHz, uint32_t& MemoryMHz) const;
};

