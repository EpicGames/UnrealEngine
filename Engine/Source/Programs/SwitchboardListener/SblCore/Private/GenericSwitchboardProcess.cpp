// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardProcess.h"


FGenericSwitchboardProcess::FCreateProcResult FGenericSwitchboardProcess::CreateProc(const FCreateProcParams& InParams)
{
	FCreateProcResult Result;

	const FString Args = FString::Join(InParams.Args, TEXT(" "));

	Result.Handle = FPlatformProcess::CreateProc(
		*FString(InParams.Executable),
		*Args,
		InParams.bLaunchDetached,
		InParams.bLaunchMinimized,
		InParams.bLaunchReallyHidden,
		&Result.ProcessId,
		InParams.PriorityModifier,
		InParams.WorkingDirectory ? *FString(*InParams.WorkingDirectory) : nullptr,
		InParams.StdoutPipe,
		InParams.StdinPipe,
		InParams.StderrPipe
	);

	return Result;
}



FProcHandle FGenericSwitchboardProcess::CreateProc(
	const TCHAR* URL,
	const TCHAR* Parms,
	bool bLaunchDetached,
	bool bLaunchHidden,
	bool bLaunchReallyHidden,
	uint32* OutProcessID,
	int32 PriorityModifier,
	const TCHAR* OptionalWorkingDirectory,
	void* PipeWriteChild,
	void* PipeReadChild
)
{
	return FSwitchboardProcess::CreateProc(URL, Parms, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, OutProcessID, PriorityModifier, OptionalWorkingDirectory, PipeWriteChild, PipeReadChild, nullptr);
}


FProcHandle FGenericSwitchboardProcess::CreateProc(
	const TCHAR* URL,
	const TCHAR* Parms,
	bool bLaunchDetached,
	bool bLaunchHidden,
	bool bLaunchReallyHidden,
	uint32* OutProcessID,
	int32 PriorityModifier,
	const TCHAR* OptionalWorkingDirectory,
	void* PipeWriteChild,
	void* PipeReadChild,
	void* PipeStdErrChild
)
{
	const FString UrlStr(URL);
	const FString ParmsStr(Parms);
	const TArray<FStringView> ArgsArray = { FStringView(ParmsStr) };
	const FString WorkingDirectoryStr(OptionalWorkingDirectory);

	FCreateProcParams Params = {
		.Executable = UrlStr,
		.Args = ArgsArray,
		.bLaunchDetached = bLaunchDetached,
		.bLaunchMinimized = bLaunchHidden,
		.bLaunchReallyHidden = bLaunchReallyHidden,
		.PriorityModifier = PriorityModifier,
		.StdinPipe = PipeReadChild,
		.StdoutPipe = PipeWriteChild,
		.StderrPipe = PipeStdErrChild,
	};

	if (OptionalWorkingDirectory)
	{
		Params.WorkingDirectory = WorkingDirectoryStr;
	}

	FCreateProcResult Result = FSwitchboardProcess::CreateProc(Params);
	if (OutProcessID)
	{
		*OutProcessID = Result.ProcessId;
	}

	return Result.Handle;
}
