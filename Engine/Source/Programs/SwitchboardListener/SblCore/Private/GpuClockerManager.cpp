// Copyright Epic Games, Inc. All Rights Reserved.

#include "GpuClockerManager.h"

#include "GpuClocker.h"
#include "Misc/CommandLine.h"
#include "SBLHelperClient.h"
#include "SwitchboardListenerApp.h" // To include LogSwitchboard


FGpuClockerManager::FGpuClockerManager()
	: SBLHelper(MakeShared<FSBLHelperClient>())
{ 
}

FGpuClockerManager::~FGpuClockerManager()
{
	// Unlock clocks when exiting. We do not expect SBL to be closed while it is managing processes but if it happens
	// it should try to leave them in a normal state. In case of abnormal termination, this code won't run
	// and the gpus will be left as they were.
	if (bLockManagedLocally)
	{
		FGpuClocker GpuClocker;
		GpuClocker.UnlockGpuClocks();
	}
}


bool FGpuClockerManager::LockGpuClocksForPid(uint32 Pid)
{
	// Add the Pid to the set. We do this anyway whether we fully succeed or not in the code below
	// to account for the case of a GPU succeeding and another one not succeeding. In that case,
	// we want to proceeed and not get into a situation of a GPU actually getting locked but never unlocking it
	// because we thought it failed.
	LockingPids.Add(Pid);
	
	// Attempt to lock via the SBL Helper if we're not locally managing the lock.
	if (!bLockManagedLocally && LockGpuClocksUsingSBLHelper(Pid))
	{
		return true;
	}

	// If SBLH is not available then we attempt to do it locally.
	UE_LOGF(LogSwitchboard, Display, "Locking Gpu Clocks");
	bLockManagedLocally = true;

	FGpuClocker GpuClocker;
	const bool bLocked = GpuClocker.LockGpuClocks();

	if (!bLocked)
	{
		UE_LOGF(LogSwitchboard, Warning, "Unable to lock Gpu clocks as requested in Switchboard settings for process with id %d. Please run SwitchboardListenerHelper "
			"or SwitchboardListener with elevated privileges because NVML requires this in order to lock the Gpu clocks.", Pid);
	}

	return bLocked;
}


void FGpuClockerManager::PidEnded(uint32 Pid)
{
	// Remove the ended pid from our list that keeps the gpus locked.
	// If nothing was removed, then we just return as there is nothing to do.
	if (!LockingPids.Remove(Pid))
	{
		return;
	}

	// If we just emptied the pids and are locally managing the lock, then we locally unlock the gpu clocks.
	if (bLockManagedLocally && LockingPids.IsEmpty())
	{
		UE_LOGF(LogSwitchboard, Display, "Unlocking Gpu Clocks");

		// Since we're unlocking, we cease to assert that we are managing the lock locally.
		// This makes it so that it tries SBL Helper first next time.
		bLockManagedLocally = false;

		FGpuClocker GpuClocker;
		GpuClocker.UnlockGpuClocks();
	}
}


void FGpuClockerManager::Tick()
{
	if (SBLHelper.IsValid())
	{
		SBLHelper->Tick();
	}
}


bool FGpuClockerManager::LockGpuClocksUsingSBLHelper(uint32 Pid)
{
	if (!SBLHelper.IsValid())
	{
		return false;
	}

	// Try to connect to the SBLHelper server if we haven't already
	if (!SBLHelper->IsConnected())
	{
		FSBLHelperClient::FConnectionParams ConnectionParams;

		uint16 Port = 8010; // Default tcp port

		// Apply command line port number override, if present
		{
			static uint16 CmdLinePortOverride = 0;
			static bool bCmdLinePortOverrideParsed = false;
			static bool bCmdLinePortOverrideValid = false;

			if (!bCmdLinePortOverrideParsed)
			{
				bCmdLinePortOverrideParsed = true;

				bCmdLinePortOverrideValid = FParse::Value(FCommandLine::Get(), TEXT("sblhport="), CmdLinePortOverride);
			}

			if (bCmdLinePortOverrideValid)
			{
				Port = CmdLinePortOverride;
			}
		}

		const FString HostName = FString::Printf(TEXT("localhost:%d"), Port);
		FIPv4Endpoint::FromHostAndPort(*HostName, ConnectionParams.Endpoint);

		SBLHelper->Connect(ConnectionParams);
	}

	if (!SBLHelper->IsConnected())
	{
		return false;
	}

	const bool bSentMessage = SBLHelper->LockGpuClock(Pid);

	if (!bSentMessage)
	{
		UE_LOGF(LogSwitchboard, Error, "Failed to send message to SBLHelper server to request gpu clock locking");
	}

	// We disconnect right away because launches happen only far and in between.
	SBLHelper->Disconnect();

	return true;
}

