// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


class FSBLHelperClient;


/**
 * Manages the GpuClocker, whether using local functionality when SBL is launched with elevanted privileges
 * (which is an NVML requirement to lock Gpu clocks), or the SwitchboardListenerHelper process
 */
class FGpuClockerManager
{
public:

	FGpuClockerManager();
	~FGpuClockerManager();

	/** Call periodically to perform service maintenance tasks. */
	void Tick();

	/** Locks the Gpu clocks for at least the lifetime of the given process id. */
	bool LockGpuClocksForPid(uint32 Pid);

	/** 
	 * Must be called when a Pid that this gpu clock manager may have in its internal list has ended. 
	 * When all pids that conditio the gpu clocks end, then the clocks are put back to their normal state.
	 */
	void PidEnded(uint32 Pid);

private:

	/** Requests the SBL Helper executable to lock the Gpu clocks during the lifetime of at least the given process id. */
	bool LockGpuClocksUsingSBLHelper(uint32 Pid);

private:

	/** Client interface to the Switchboard Listener Helper external process */
	TSharedPtr<FSBLHelperClient> SBLHelper;

	/** 
	 * Set of Pids that are keeping the gpus clocks locked. Only applicable when locally
	 * managed and not when using the external SBLHelper.
	 */
	TSet<uint32> LockingPids;

	/** 
	 * True when we are locally managing the locking of the Gpu clocks. It doesn't imply that the local
	 * locking succeeded, only that we are not using SBL Helper to do it for the current pids in the set.
	 */
	bool bLockManagedLocally = false;
};

