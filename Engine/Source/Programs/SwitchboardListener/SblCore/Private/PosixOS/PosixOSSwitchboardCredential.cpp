// Copyright Epic Games, Inc. All Rights Reserved.

#include "PosixOS/PosixOSSwitchboardCredential.h"
#include "SwitchboardAuth.h"
#include "SwitchboardListenerApp.h" // for LogSwitchboard

#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Logging/StructuredLog.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

#include <stdlib.h>
#include <sys/stat.h>


namespace UE::SwitchboardListener::Private
{
	struct FJsonCredential
	{
		static constexpr FStringView UserFieldName = TEXTVIEW("user");
		static constexpr FStringView BlobFieldName = TEXTVIEW("blob");

		FString UserField;
		FString BlobField;


		FString ToJsonString() const
		{
			FString Result;

			TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> JsonWriter =
				TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&Result);

			JsonWriter->WriteObjectStart();
			JsonWriter->WriteValue(UserFieldName, UserField);
			JsonWriter->WriteValue(BlobFieldName, BlobField);
			JsonWriter->WriteObjectEnd();
			ensure(JsonWriter->Close());

			return Result;
		}


		static TOptional<FJsonCredential> FromJsonString(FStringView InJsonString)
		{
			TSharedRef<TJsonReader<TCHAR>> Reader =
				TJsonReaderFactory<>::CreateFromView(InJsonString);

			TSharedPtr<FJsonObject> JsonData;
			if (!ensure(FJsonSerializer::Deserialize(Reader, JsonData)))
			{
				UE_LOGFMT(LogSwitchboard, Error, "{Func}: Failed to deserialize: {JsonString}",
					__FUNCTION__, InJsonString);
				return {};
			}

			FJsonCredential Result;
			if (!ensure(JsonData->TryGetStringField(UserFieldName, Result.UserField)))
			{
				UE_LOGFMT(LogSwitchboard, Error, "{Func}: Failed to get field '{FieldName}'",
					__FUNCTION__, UserFieldName);
				return {};
			}

			if (!ensure(JsonData->TryGetStringField(BlobFieldName, Result.BlobField)))
			{
				UE_LOGFMT(LogSwitchboard, Error, "{Func}: Failed to get field '{FieldName}'",
					__FUNCTION__, BlobFieldName);
				return {};
			}

			return Result;
		}
	};


	class FPosixCredential : public ICredential
	{
	public:
		FPosixCredential(const FJsonCredential& InJsonCredential)
			: JsonCredential(InJsonCredential)
		{
		}

		virtual FStringView GetUser() override
		{
			return JsonCredential.UserField;
		}


		virtual FStringView GetBlob() override
		{
			return JsonCredential.BlobField;
		}

	private:
		FJsonCredential JsonCredential;
	};


	FString GetCredentialFilePath(FStringView InCredentialName)
	{
		const FString SblDataDir = GetSwitchboardDataDirectory();

		const FString FileName = FString::Printf(TEXT("credential_%*s.json"),
			InCredentialName.Len(), InCredentialName.GetData());

		return SblDataDir / FileName;
	}

	TSharedPtr<ICredential> FPosixCredentialManager::LoadCredential(FStringView InCredentialName)
	{
		const FString FilePath = GetCredentialFilePath(InCredentialName);
		if (!FPaths::FileExists(FilePath))
		{
			UE_LOGFMT(LogSwitchboard, Verbose, "Credential at {Path} does not exist", FilePath);
			return {};
		}

		if (!ensure(UE::SwitchboardListener::Private::CheckCredentialPermissions(FilePath)))
		{
			UE_LOGFMT(LogSwitchboard, Error, "Credential file at {Path} has incorrect permissions!", FilePath);
		}

		FString CredentialStr;
		if (!ensure(FFileHelper::LoadFileToString(CredentialStr, *FilePath)))
		{
			UE_LOGFMT(LogSwitchboard, Error, "Unable to read credential file at {Path}", FilePath);
			return {};
		}

		TOptional<FJsonCredential> LoadedCredential = FJsonCredential::FromJsonString(CredentialStr);
		if (!ensure(LoadedCredential))
		{
			UE_LOGFMT(LogSwitchboard, Error, "Error parsing file at {Path}", FilePath);
			return {};
		}

		return MakeShared<FPosixCredential>(*LoadedCredential);
	}


	bool FPosixCredentialManager::SaveCredential(FStringView InCredentialName, FStringView InUser, FStringView InBlob)
	{
		// Construct the temporary file path template for mkstemp.
		const FString DestFilePath = GetCredentialFilePath(InCredentialName);
		const FString DestDir = FPaths::GetPath(DestFilePath);
		const FUtf8String TempFilePathTemplate(DestDir / FString("sbltempXXXXXX"));

		// Copy into a mutable array for mkstemp(), which modifies its argument
		// in-place, replacing 'XXXXXX' with randomly-generated unique characters.
		TArray<UTF8CHAR> TempFilePathU8(*TempFilePathTemplate, TempFilePathTemplate.Len() + 1);

		// Create the directory tree with appropriate permissions.
		const mode_t OriginalUmask = umask(~CredentialDirPermissionFlags);
		ON_SCOPE_EXIT{ umask(OriginalUmask); };

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		PlatformFile.CreateDirectoryTree(*DestDir);

		// Create the temporary file with appropriately restrictive permissions.
		// POSIX.1-2008 says this umask should be redundant, but just to be safe.
		umask(~CredentialFilePermissionFlags);
		int TempFd = mkstemp(reinterpret_cast<char*>(TempFilePathU8.GetData()));
		if (!ensure(TempFd != -1))
		{
			const int ErrorCode = errno;
			UE_LOGFMT(LogSwitchboard, Error, "{Func}: mkstemp failed (errno: {errno}; path: '{TempFilePath}')",
				__FUNCTION__, ErrorCode, TempFilePathU8);
			return false;
		}

		const FJsonCredential JsonCredential{ FString(InUser), FString(InBlob) };
		const FUtf8String JsonString(JsonCredential.ToJsonString());

		int ErrorCode = 0;
		int32 TotalBytesWritten = 0;
		while (TotalBytesWritten < JsonString.Len())
		{
			const SSIZE_T Written = write(TempFd,
				(*JsonString) + TotalBytesWritten,
				JsonString.Len() - TotalBytesWritten);

			if (Written <= 0)
			{
				ErrorCode = errno;
				break;
			}

			TotalBytesWritten += Written;
		}

		const int CloseResult = close(TempFd);
		if (!ensure(CloseResult == 0))
		{
			const int CloseErrorCode = errno;
			UE_LOGFMT(LogSwitchboard, Warning, "{Func}: close failed (errno: {errno}; path: '{TempFilePath}')",
				__FUNCTION__, CloseErrorCode, TempFilePathU8);
		}

		// If write() failed, delete the truncated temporary file.
		if (TotalBytesWritten < JsonString.Len())
		{
			PlatformFile.DeleteFile(UTF8_TO_TCHAR(TempFilePathU8.GetData()));
			UE_LOGFMT(LogSwitchboard, Error,
				"{Func}: write failed (errno: {errno}; written {BytesWritten}/{JsonByteLen}; path: '{TempFilePath}')",
				__FUNCTION__, ErrorCode, TotalBytesWritten, JsonString.Len(), TempFilePathU8);
			return false;
		}

		// Otherwise, rename the temporary file to its intended destination.
		const int RenameResult = rename((const char*)TempFilePathU8.GetData(),
			TCHAR_TO_UTF8(*DestFilePath));
		if (!ensure(RenameResult == 0))
		{
			ErrorCode = errno;
			PlatformFile.DeleteFile(UTF8_TO_TCHAR(TempFilePathU8.GetData()));
			UE_LOGFMT(LogSwitchboard, Warning, "{Func}: rename failed (errno: {errno}; path: '{TempFilePath}' -> '{DestFilePath}')",
				__FUNCTION__, ErrorCode, TempFilePathU8, DestFilePath);
			return false;
		}

		return true;
	}
}; // namespace UE::SwitchboardListener::Private
