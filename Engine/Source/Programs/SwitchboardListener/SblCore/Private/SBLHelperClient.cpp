// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBLHelperClient.h"

#include "Common/TcpSocketBuilder.h"
#include "Dom/JsonValue.h"
#include "JsonObjectConverter.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"



DEFINE_LOG_CATEGORY(LogSBLHelperClient);

const int32 FSBLHelperClient::CompatibleVersion[3] = { 1, 0, 0 };

FSBLHelperClient::~FSBLHelperClient()
{
	Disconnect();
}

bool FSBLHelperClient::IsConnected() 
{
	int32 BytesSent;
	return Socket && Socket->Send(nullptr, 0, BytesSent);
}

bool FSBLHelperClient::Connect(const FConnectionParams& InConnectionParams)
{
	check(IsInGameThread());
	
	// Cache connection parameters
	ConnectionParams = InConnectionParams;

	// Make sure we have released everything related to any previous connection attempts
	Disconnect();
	check(!Socket);

	// Create socket for the new connection

	Socket = FTcpSocketBuilder(TEXT("SBLHelperSocket"));
	check(Socket);

	// Attempt to connect
	{
		const bool bConnected = Socket->Connect(*ConnectionParams.Endpoint.ToInternetAddr());

		if (!bConnected)
		{
			Disconnect();
			return false;
		}
	}

	// Confirm that this is actually a SBL server by parsing its first packet(s), if any.
	constexpr float TimeoutSeconds = 0.5;
	ParseIncomingMessages(TimeoutSeconds);

	// If the hello packet arrived, ServerVersion must have been populated, which we verify below.
	{
		TArray<FString> VersionStringArray;
		ServerVersion.ParseIntoArray(VersionStringArray, TEXT("."));

		if (VersionStringArray.Num() != 3)
		{
			UE_LOGF(LogSBLHelperClient, Error, "Invalid SBLHelperService version '%ls' at endpoint %ls", 
				*ServerVersion,
				*ConnectionParams.Endpoint.ToString()
			);

			Disconnect();
			return false;
		}

		const int32 VersionArray[3] = {
			FCString::Atoi(*VersionStringArray[0]),
			FCString::Atoi(*VersionStringArray[1]),
			FCString::Atoi(*VersionStringArray[2]),
		};

		// We'll only look at major and minor.
		if ((VersionArray[0] != CompatibleVersion[0]) 
			|| (VersionArray[1] < CompatibleVersion[1]))
		{
			UE_LOGF(LogSBLHelperClient, Error, "Incompatible SBLHelperService version '%ls' at endpoint %ls. Minimum required is %d.%d.%d", 
				*ServerVersion,
				*ConnectionParams.Endpoint.ToString(),
				CompatibleVersion[0],
				CompatibleVersion[1],
				CompatibleVersion[2]
			);

			Disconnect();
			return false;
		}
	}

	// Ok, we're good to go.
	UE_LOGF(LogSBLHelperClient, Display, "Connected to SBLHelperService version '%ls' at endpoint %ls",
		*ServerVersion,
		*ConnectionParams.Endpoint.ToString()
	);

	return true;
}

void FSBLHelperClient::HandleCmdHello(TSharedPtr<FJsonObject>& Json)
{
	check(Json.IsValid());

	// We use "sblhver" instead of just "ver" as a magic value in disguise.
	TSharedPtr<FJsonValue> VersionField = Json->TryGetField(TEXT("sblhver"));

	if (!VersionField.IsValid())
	{
		return;
	}

	ServerVersion = VersionField->AsString();
}

void FSBLHelperClient::ParseMessage(const FString& Message)
{
	// All messages are in Json format

	TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(Message);
	TSharedPtr<FJsonObject> JsonData;

	if (!FJsonSerializer::Deserialize(Reader, JsonData))
	{
		return;
	}

	// All messages must have a cmd field.
	TSharedPtr<FJsonValue> CmdField = JsonData->TryGetField(TEXT("cmd"));
	if (!CmdField.IsValid())
	{
		return;
	}

	const FString Cmd = CmdField->AsString();

	// Each command has its own handler
	if (Cmd.Equals("hello"))
	{
		HandleCmdHello(JsonData);
	}
	else
	{
		UE_LOGF(LogSBLHelperClient, Error, "Unhandled command '%ls'", *Cmd);
	}
}

void FSBLHelperClient::ParseIncomingMessages(const float TimeoutSeconds)
{
	for (const FString& Message : ReceiveMessages(TimeoutSeconds))
	{
		ParseMessage(Message);
	}
}

void FSBLHelperClient::Tick()
{
	if (!Socket)
	{
		return;
	}

	// We parse and handle all incoming messages.
	ParseIncomingMessages();
}

TArray<FString> FSBLHelperClient::ReceiveMessages(const float TimeoutSeconds)
{
	TArray<FString> Messages;

	if (!Socket)
	{
		return Messages;
	}

	const double TimeoutTime = FPlatformTime::Seconds() + TimeoutSeconds;

	do // Execute at least once
	{
		uint32 PendingDataSize = 0;

		while (Socket->HasPendingData(PendingDataSize))
		{
			// Read 
			TArray<uint8> Buffer;
			Buffer.AddUninitialized(PendingDataSize);
			int32 BytesRead = 0;

			if (!Socket->Recv(Buffer.GetData(), PendingDataSize, BytesRead, ESocketReceiveFlags::None))
			{
				UE_LOGF(LogSBLHelperClient, Error, "Error while receiving data via endpoint %ls", 
					*ConnectionParams.Endpoint.ToString()
				);

				continue;
			}

			for (int32 ByteIdx = 0; ByteIdx < BytesRead; ++ByteIdx)
			{
				ReceiveBuffer.Add(Buffer[ByteIdx]);

				if (Buffer[ByteIdx] == '\x00')
				{
					Messages.Emplace(UTF8_TO_TCHAR(ReceiveBuffer.GetData()));

					// Clear the reception buffer so that it only contains the next partial message
					ReceiveBuffer.Empty();
				}
			}
		}

		// If we already got at least one message, we can return.
		if (Messages.Num())
		{
			return Messages;
		}

		// We don't wait more that the given timeout
		const double CurrentTime = FPlatformTime::Seconds();
		const double RemainingTimeBeforeTimeout = TimeoutTime - CurrentTime;

		if (RemainingTimeBeforeTimeout <= 0)
		{
			break;
		}

		// Sleep for a bit but not to exceed our timeout time.
		const double SleepSeconds = FMath::Min(0.025, RemainingTimeBeforeTimeout);
		FPlatformProcess::SleepNoStats(SleepSeconds);

	} while (true);

	return Messages;
}

uint32 FSBLHelperClient::GetNextPacketId()
{
	NextPacketId++;

	// Avoid 0, which is reserved for no packet id.

	if (NextPacketId == 0)
	{
		NextPacketId = 1;
	}

	return NextPacketId;
}

FString FSBLHelperClient::CreateMessage(const FString& Cmd, const TMap<FString, FString>& AdditionalFields)
{
	FString Message;

	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter 
		= TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Message);

	JsonWriter->WriteObjectStart();

	// message id
	JsonWriter->WriteValue(TEXT("id"), FString::Printf(TEXT("%u"), GetNextPacketId()));

	// cmd
	JsonWriter->WriteValue(TEXT("cmd"), Cmd);

	for (const auto& Value : AdditionalFields)
	{
		JsonWriter->WriteValue(Value.Key, Value.Value);
	}

	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();

	return Message;
}

void FSBLHelperClient::Disconnect()
{
	// Send by message
	SendMessage(CreateMessage(TEXT("bye"), TMap<FString, FString>()));

	// Clear buffer so that we don't try to use leftover data on the next connection
	ReceiveBuffer.Empty();

	// Clear the version as well, for the same reason.
	ServerVersion.Empty();

	// Close and destroy socket
	if (Socket)
	{
		Socket->Close();
		ISocketSubsystem::Get()->DestroySocket(Socket);
		Socket = nullptr;
	}
}

bool FSBLHelperClient::LockGpuClock(const uint32 Pid)
{
	// Send the lock command and the related process pid.

	TMap<FString, FString> AdditionalFields;
	AdditionalFields.Emplace(TEXT("pid"), FString::Printf(TEXT("%u"), Pid));

	return SendMessage(CreateMessage(TEXT("lock"), AdditionalFields));
}

bool FSBLHelperClient::SendMessage(const FString& Message)
{
	if (!Socket)
	{
		return false;
	}

	UE_LOGF(LogSBLHelperClient, Verbose, "Sending message: %ls", *Message);

	// Send the message over the socket. 
	// Note the "+ 1" to include the null terminator, which is the message delimiter in the SBLHelper protocol.

	int32 BytesSent = 0;
	return Socket->Send((uint8*)TCHAR_TO_UTF8(*Message), Message.Len() + 1, BytesSent);
}

FString FSBLHelperClient::GetStatus()
{
	if (!IsConnected())
	{
		return TEXT("Disconnected");
	}

	return TEXT("Connected");
}

