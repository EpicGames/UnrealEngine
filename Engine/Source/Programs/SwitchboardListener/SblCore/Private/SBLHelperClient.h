// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSBLHelperClient, Log, All);

class FJsonObject;
class FSocket;

/** Class in charge of communicating with the SwitchboardListenerHelper server */
class FSBLHelperClient
{
public:
	
	/** Structure holding the desired connection parameters, such as IP and Port */
	struct FConnectionParams
	{
		/** Server end point to connect to */
		FIPv4Endpoint Endpoint;
	};

	/** Destructor */
	~FSBLHelperClient();

	/** Returns true if currently connected to the SBLHelper server */
	bool IsConnected();

	/** 
	 * Connects to the SBLHelper server specified in the given connection parameters. 
	 * If already connected, it will disconnect before attempting the new connection.
	 * 
	 * @return True if it was able to establish the connection to a valid server.
	 */
	bool Connect(const FConnectionParams& InConnectionParams);

	/** Disconnects from the SBLHelper server. Can be called even if already disconnected. */
	void Disconnect();

	/** 
	 * Requests the server to lock the gpu and memory clocks to the maximum allowed. 
	 * 
	 * @param Pid Proces ID to which the lock gpu clock request is tied to.
	 * 
	 * @return True if the message was sent over the socket. 
	 */
	bool LockGpuClock(const uint32 Pid);

	/** Returns a string describing the status of the server */
	FString GetStatus();

	/** Call this periodically for tasks such as parsing the incoming messages from the server */
	void Tick();

private:

	/** 
	 * Receives messages from the socket into an array. 
	 * 
	 * @param TimeoutSeconds Maximum time to wait for at least one message to be completely received.
	 * 
	 * @return Array of received messages.
	 */
	TArray<FString> ReceiveMessages(const float TimeoutSeconds = 0);

	/** 
	 * Parses the given message and calls the respective handlers 
	 * 
	 * @param InMessage The message to parse.
	 */
	void ParseMessage(const FString& InMessage);

	/** 
	 * Checks the socket for new messages and calls ParseMessage on each.
	 * 
	 * @param TimeoutSeconds The maximum time to wait for at least one message to be completely received.
	 */
	void ParseIncomingMessages(const float TimeoutSeconds = 0);

	/** Handles the 'Hello' command from the server */
	void HandleCmdHello(TSharedPtr<FJsonObject>& Json);

	/** 
	 * Creates a new message for the given command and additional fields 
	 * 
	 * @param Cmd The command to create this message for.
	 * @param AdditionalFields Any additional fields or parameters to include in the message.
	 * 
	 * @return The created message, ready to be sent over the socket.
	 */
	FString CreateMessage(const FString& Cmd, const TMap<FString, FString>& AdditionalFields);

	/** Sends the given message over the socket, if connected. */
	bool SendMessage(const FString& Message);

	/** Gets the next packet id to use when creating a new message. Monotonically increments. Skips zero. */
	uint32 GetNextPacketId();

private:

	/** The minimum compatible SBLHelper server version.  */
	static const int32 CompatibleVersion[3];

	/** Socket for this client to communicate with the server */
	FSocket* Socket = nullptr;

	/** Next packet id when creating a new message to be sent */
	uint32 NextPacketId = 1;

	/** Buffer to hold partial messages received over the socket from the server */
	TArray<uint8> ReceiveBuffer;

	/** Cached connection parameters. Mostly used for logging purposes. */
	FConnectionParams ConnectionParams;

	/** The current server version that this client is connected to. */
	FString ServerVersion;
};

