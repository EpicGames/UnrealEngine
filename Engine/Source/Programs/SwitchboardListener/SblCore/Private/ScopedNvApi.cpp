// Copyright Epic Games, Inc. All Rights Reserved.

#if PLATFORM_WINDOWS

#include "ScopedNvApi.h"

#include "Async/RecursiveMutex.h"
#include "SwitchboardListenerApp.h"

#include "nvapi.h"


/** Contains static data relevant to FScopedNvApi */
class FScopedNvApi::Statics
{
public:

	/** Global lock for NvApi usage */
	static UE::FRecursiveMutex RecursiveMutex;

	/** Keeps track of failed unloads. Useful to recover from potential NvApi unloading errors */
	static std::atomic<int32> OwedUnloads;

	/** Used to repeatedly show NvAPI initialization errors. Once is enough. */
	static bool bErrorLoggedOnce;

	/** Cache of the GPU physical handles */
	static TArray<NvPhysicalGpuHandle> PhysicalGpuHandles;

	/** Keeps count of the number of instances of this class. Used to manage shared resources. */
	static std::atomic<int32> InstanceCount;

	/** Delegate for when new instances are created */
	static FOnNvApiInstantiated OnNvApiInstantiated;
};


// Initialize static variables of FScopedNvApi::Statics
bool                               FScopedNvApi::Statics::bErrorLoggedOnce = false;
std::atomic<int32>                 FScopedNvApi::Statics::OwedUnloads = 0;
std::atomic<int32>                 FScopedNvApi::Statics::InstanceCount = 0;
UE::FRecursiveMutex                FScopedNvApi::Statics::RecursiveMutex;
TArray<NvPhysicalGpuHandle>        FScopedNvApi::Statics::PhysicalGpuHandles;
FScopedNvApi::FOnNvApiInstantiated FScopedNvApi::Statics::OnNvApiInstantiated;


FScopedNvApi::FOnNvApiInstantiated& FScopedNvApi::GetOnNvApiInstantiated()
{
	return Statics::OnNvApiInstantiated;
}


FScopedNvApi::FScopedNvApi()
{
	UE::TUniqueLock<UE::FRecursiveMutex> Lock(Statics::RecursiveMutex);

	// Count this new instance
	Statics::InstanceCount++;

	// Initialize NvAPI.

	if (Statics::OwedUnloads)
	{
		Statics::OwedUnloads--;
		bIsNvApiInitialized = true; // If it isn't true, the calls to the api will simply fail.
	}
	else
	{
		const NvAPI_Status Result = NvAPI_Initialize();

		if (Result == NVAPI_OK)
		{
			bIsNvApiInitialized = true;
		}
		else if (!Statics::bErrorLoggedOnce)
		{
			Statics::bErrorLoggedOnce = false;

			NvAPI_ShortString ErrorString;
			NvAPI_GetErrorMessage(Result, ErrorString);
			UE_LOGF(LogSwitchboard, Error, "NvAPI_Initialize failed. Error: %ls", ANSI_TO_TCHAR(ErrorString));
		}
	}

	if (Statics::InstanceCount == 1)
	{
		UE_LOGF(LogSwitchboard, VeryVerbose, "FScopedNvApi InstanceCount incremented to 1");
	}

	// Let others know that a new instance of this class was instantiated.
	Statics::OnNvApiInstantiated.Broadcast();
}

FScopedNvApi::~FScopedNvApi()
{
	UE::TUniqueLock<UE::FRecursiveMutex> Lock(Statics::RecursiveMutex);

	// One less instance alive
	Statics::InstanceCount--;

	check(Statics::InstanceCount >= 0);

	// Clear cache of PhysicalGpuHandles when we go back to zero, forcing a re-cache when
	// a future instance tries to get a gpu count or read gpu stats.
	if (!Statics::InstanceCount)
	{
		UE_LOGF(LogSwitchboard, VeryVerbose, "FScopedNvApi InstanceCount decremented to 0");
		Statics::PhysicalGpuHandles.Reset(); // Keep the memory.
	}

	// Only unload NvApi if it was initialized
	if (bIsNvApiInitialized)
	{
		const NvAPI_Status Result = NvAPI_Unload();

		if (Result == NVAPI_OK)
		{
			bIsNvApiInitialized = false;
		}
		else
		{
			// For this unexpected event we try to compensate by not loading next time.

			Statics::OwedUnloads++;

			NvAPI_ShortString ErrorString;
			NvAPI_GetErrorMessage(Result, ErrorString);
			UE_LOGF(LogSwitchboard, Error, "NvAPI_Unload unexpectedly failed. Error: %ls", ANSI_TO_TCHAR(ErrorString));
		}
	}

}

/** Returns true if the NvAPI was initialized successfully */
bool FScopedNvApi::IsNvApiInitialized() const
{
	return bIsNvApiInitialized;
}

void FScopedNvApi::FillOutSyncTopologies(TArray<FSyncTopo>& SyncTopos) const
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FillOutSyncTopologies);

	UE::TUniqueLock<UE::FRecursiveMutex> Lock(Statics::RecursiveMutex);

	if (!IsNvApiInitialized())
	{
		return;
	}

	// Normally there is a single sync card. BUT an RTX Server could have more, and we need to account for that.

	// Detect sync cards

	NvU32 GSyncCount = 0;
	NvGSyncDeviceHandle GSyncHandles[NVAPI_MAX_GSYNC_DEVICES];
	NvAPI_GSync_EnumSyncDevices(GSyncHandles, &GSyncCount); // GSyncCount will be zero if error, so no need to check error.

	for (NvU32 GSyncIdx = 0; GSyncIdx < GSyncCount; GSyncIdx++)
	{
		NvU32 GSyncGPUCount = 0;
		NvU32 GSyncDisplayCount = 0;

		// gather info first with null data pointers, just to get the count and subsequently allocate necessary memory.
		{
			const NvAPI_Status Result = NvAPI_GSync_GetTopology(GSyncHandles[GSyncIdx], &GSyncGPUCount, nullptr, &GSyncDisplayCount, nullptr);

			if (Result != NVAPI_OK)
			{
				NvAPI_ShortString ErrorString;
				NvAPI_GetErrorMessage(Result, ErrorString);
				UE_LOGF(LogSwitchboard, Warning, "NvAPI_GSync_GetTopology failed. Error: %ls", ANSI_TO_TCHAR(ErrorString));
				continue;
			}
		}

		// allocate memory for data
		TArray<NV_GSYNC_GPU> GSyncGPUs;
		TArray<NV_GSYNC_DISPLAY> GSyncDisplays;
		{
			GSyncGPUs.SetNumUninitialized(GSyncGPUCount, EAllowShrinking::No);

			for (NvU32 GSyncGPUIdx = 0; GSyncGPUIdx < GSyncGPUCount; GSyncGPUIdx++)
			{
				GSyncGPUs[GSyncGPUIdx].version = NV_GSYNC_GPU_VER;
			}

			GSyncDisplays.SetNumUninitialized(GSyncDisplayCount, EAllowShrinking::No);

			for (NvU32 GSyncDisplayIdx = 0; GSyncDisplayIdx < GSyncDisplayCount; GSyncDisplayIdx++)
			{
				GSyncDisplays[GSyncDisplayIdx].version = NV_GSYNC_DISPLAY_VER;
			}
		}

		// get real info
		{
			const NvAPI_Status Result = NvAPI_GSync_GetTopology(GSyncHandles[GSyncIdx], &GSyncGPUCount, GSyncGPUs.GetData(), &GSyncDisplayCount, GSyncDisplays.GetData());

			if (Result != NVAPI_OK)
			{
				NvAPI_ShortString ErrorString;
				NvAPI_GetErrorMessage(Result, ErrorString);
				UE_LOGF(LogSwitchboard, Warning, "NvAPI_GSync_GetTopology failed. Error: %ls", ANSI_TO_TCHAR(ErrorString));
				continue;
			}
		}

		// Build outbound structure

		FSyncTopo SyncTopo;

		for (NvU32 GpuIdx = 0; GpuIdx < GSyncGPUCount; GpuIdx++)
		{
			FSyncGpu SyncGpu;

			SyncGpu.bIsSynced = GSyncGPUs[GpuIdx].isSynced;
			SyncGpu.Connector = int32(GSyncGPUs[GpuIdx].connector);

			SyncTopo.SyncGpus.Emplace(SyncGpu);
		}

		for (NvU32 DisplayIdx = 0; DisplayIdx < GSyncDisplayCount; DisplayIdx++)
		{
			FSyncDisplay SyncDisplay;

			switch (GSyncDisplays[DisplayIdx].syncState)
			{
			case NVAPI_GSYNC_DISPLAY_SYNC_STATE_UNSYNCED:
				SyncDisplay.SyncState = TEXT("Unsynced");
				break;
			case NVAPI_GSYNC_DISPLAY_SYNC_STATE_SLAVE:
				SyncDisplay.SyncState = TEXT("Follower");
				break;
			case NVAPI_GSYNC_DISPLAY_SYNC_STATE_MASTER:
				SyncDisplay.SyncState = TEXT("Leader");
				break;
			default:
				SyncDisplay.SyncState = TEXT("Unknown");
				break;
			}

			// get color information for each display
			{
				NV_COLOR_DATA ColorData;

				ColorData.version = NV_COLOR_DATA_VER;
				ColorData.cmd = NV_COLOR_CMD_GET;
				ColorData.size = sizeof(NV_COLOR_DATA);

				const NvAPI_Status Result = NvAPI_Disp_ColorControl(GSyncDisplays[DisplayIdx].displayId, &ColorData);

				if (Result == NVAPI_OK)
				{
					SyncDisplay.Bpc = ColorData.data.bpc;
					SyncDisplay.Depth = ColorData.data.depth;
					SyncDisplay.ColorFormat = ColorData.data.colorFormat;
				}
			}

			SyncTopo.SyncDisplays.Emplace(SyncDisplay);
		}

		// Sync Status Parameters
		{
			NV_GSYNC_STATUS_PARAMS GSyncStatusParams;
			GSyncStatusParams.version = NV_GSYNC_STATUS_PARAMS_VER;

			const NvAPI_Status Result = NvAPI_GSync_GetStatusParameters(GSyncHandles[GSyncIdx], &GSyncStatusParams);

			if (Result != NVAPI_OK)
			{
				NvAPI_ShortString ErrorString;
				NvAPI_GetErrorMessage(Result, ErrorString);
				UE_LOGF(LogSwitchboard, Warning, "NvAPI_GSync_GetStatusParameters failed. Error: %ls", ANSI_TO_TCHAR(ErrorString));
				continue;
			}

			SyncTopo.SyncStatusParams.RefreshRate = GSyncStatusParams.refreshRate;
			SyncTopo.SyncStatusParams.HouseSyncIncoming = GSyncStatusParams.houseSyncIncoming;
			SyncTopo.SyncStatusParams.bHouseSync = !!GSyncStatusParams.bHouseSync;
			SyncTopo.SyncStatusParams.bInternalSecondary = GSyncStatusParams.bInternalSlave;
		}

		// Sync Control Parameters
		{
			NV_GSYNC_CONTROL_PARAMS GSyncControlParams;
			GSyncControlParams.version = NV_GSYNC_CONTROL_PARAMS_VER;

			const NvAPI_Status Result = NvAPI_GSync_GetControlParameters(GSyncHandles[GSyncIdx], &GSyncControlParams);

			if (Result != NVAPI_OK)
			{
				NvAPI_ShortString ErrorString;
				NvAPI_GetErrorMessage(Result, ErrorString);
				UE_LOGF(LogSwitchboard, Warning, "NvAPI_GSync_GetControlParameters failed. Error: %ls", ANSI_TO_TCHAR(ErrorString));
				continue;
			}

			SyncTopo.SyncControlParams.bInterlaced = !!GSyncControlParams.interlaceMode;
			SyncTopo.SyncControlParams.bSyncSourceIsOutput = !!GSyncControlParams.syncSourceIsOutput;
			SyncTopo.SyncControlParams.Interval = GSyncControlParams.interval;
			SyncTopo.SyncControlParams.Polarity = GSyncControlParams.polarity;
			SyncTopo.SyncControlParams.Source = GSyncControlParams.source;
			SyncTopo.SyncControlParams.VMode = GSyncControlParams.vmode;

			SyncTopo.SyncControlParams.SyncSkew.MaxLines = GSyncControlParams.syncSkew.maxLines;
			SyncTopo.SyncControlParams.SyncSkew.MinPixels = GSyncControlParams.syncSkew.minPixels;
			SyncTopo.SyncControlParams.SyncSkew.NumLines = GSyncControlParams.syncSkew.numLines;
			SyncTopo.SyncControlParams.SyncSkew.NumPixels = GSyncControlParams.syncSkew.numPixels;

			SyncTopo.SyncControlParams.StartupDelay.MaxLines = GSyncControlParams.startupDelay.maxLines;
			SyncTopo.SyncControlParams.StartupDelay.MinPixels = GSyncControlParams.startupDelay.minPixels;
			SyncTopo.SyncControlParams.StartupDelay.NumLines = GSyncControlParams.startupDelay.numLines;
			SyncTopo.SyncControlParams.StartupDelay.NumPixels = GSyncControlParams.startupDelay.numPixels;
		}

		SyncTopos.Emplace(SyncTopo);
	}
}


void FScopedNvApi::FillOutDriverVersion(FSyncStatus& SyncStatus) const
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FillOutDriverVersion);

	UE::TUniqueLock<UE::FRecursiveMutex> Lock(Statics::RecursiveMutex);

	if (!IsNvApiInitialized())
	{
		return;
	}

	NvU32 DriverVersion;
	NvAPI_ShortString BuildBranchString;

	const NvAPI_Status Result = NvAPI_SYS_GetDriverAndBranchVersion(&DriverVersion, BuildBranchString);

	if (Result != NVAPI_OK)
	{
		NvAPI_ShortString ErrorString;
		NvAPI_GetErrorMessage(Result, ErrorString);
		UE_LOGF(LogSwitchboard, Warning, "NvAPI_SYS_GetDriverAndBranchVersion failed. Error: %ls", ANSI_TO_TCHAR(ErrorString));
		return;
	}

	SyncStatus.DriverVersion = DriverVersion;
	SyncStatus.DriverBranch = UTF8_TO_TCHAR(BuildBranchString);
}


uint32 FScopedNvApi::GetGpuCount()
{
	UE::TUniqueLock<UE::FRecursiveMutex> Lock(Statics::RecursiveMutex);

	if (!IsNvApiInitialized())
	{
		return 0;
	}

	// Count lazily.
	if (!Statics::PhysicalGpuHandles.Num())
	{
		CacheGpuHandles();
	}

	return Statics::PhysicalGpuHandles.Num();
}


void FScopedNvApi::FillOutPhysicalGpuStats(FSyncStatus & SyncStatus, bool bGetUtilizations, bool bGetClocks, bool bGetTemperatures)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FillOutPhysicalGpuStats);

	if (!bGetUtilizations && !bGetClocks && !bGetTemperatures)
	{
		return;
	}

	UE::TUniqueLock<UE::FRecursiveMutex> Lock(Statics::RecursiveMutex);

	if (!IsNvApiInitialized())
	{
		return;
	}

	uint32 PhysicalGpuCount = GetGpuCount();

	SyncStatus.GpuUtilization.SetNumUninitialized(PhysicalGpuCount);
	SyncStatus.GpuCoreClocksKhz.SetNumUninitialized(PhysicalGpuCount);
	SyncStatus.GpuTemperature.SetNumUninitialized(PhysicalGpuCount);

	for (NvU32 PhysicalGpuIdx = 0; PhysicalGpuIdx < PhysicalGpuCount; ++PhysicalGpuIdx)
	{
		SyncStatus.GpuUtilization[PhysicalGpuIdx] = -1;
		SyncStatus.GpuCoreClocksKhz[PhysicalGpuIdx] = -1;
		SyncStatus.GpuTemperature[PhysicalGpuIdx] = MIN_int32;

		const NvPhysicalGpuHandle& PhysicalGpu = Statics::PhysicalGpuHandles[PhysicalGpuIdx];

		NV_GPU_DYNAMIC_PSTATES_INFO_EX PstatesInfo;
		PstatesInfo.version = NV_GPU_DYNAMIC_PSTATES_INFO_EX_VER;
		NvAPI_Status NvResult = NvAPI_GPU_GetDynamicPstatesInfoEx(PhysicalGpu, &PstatesInfo);
		if (NvResult == NVAPI_OK)
		{
			// FIXME: NV_GPU_UTILIZATION_DOMAIN_ID enum is missing in our nvapi.h, but documented elsewhere.
			//const int8 UtilizationDomain = NVAPI_GPU_UTILIZATION_DOMAIN_GPU;
			const int8 UtilizationDomain = 0;
			if (PstatesInfo.utilization[UtilizationDomain].bIsPresent)
			{
				SyncStatus.GpuUtilization[PhysicalGpuIdx] = PstatesInfo.utilization[UtilizationDomain].percentage;
			}
		}
		else if (NvResult == NVAPI_HANDLE_INVALIDATED)
		{
			// If the handles are not valid anymore, we cache new ones and return.
			// This query will fail but the next one should succeed.
			UE_LOGF(LogSwitchboard, Warning, "Physical GPU handles were invalidated, recaching them.");
			CacheGpuHandles();
			return;
		}
		else
		{
			UE_LOGF(LogSwitchboard, Warning, "NvAPI_GPU_GetDynamicPstatesInfoEx failed. Error code: %d", NvResult);
		}

		NV_GPU_CLOCK_FREQUENCIES ClockFreqs;
		ClockFreqs.version = NV_GPU_CLOCK_FREQUENCIES_VER;
		ClockFreqs.ClockType = NV_GPU_CLOCK_FREQUENCIES_CURRENT_FREQ;
		NvResult = NvAPI_GPU_GetAllClockFrequencies(PhysicalGpu, &ClockFreqs);
		if (NvResult == NVAPI_OK)
		{
			if (ClockFreqs.domain[NVAPI_GPU_PUBLIC_CLOCK_GRAPHICS].bIsPresent)
			{
				SyncStatus.GpuCoreClocksKhz[PhysicalGpuIdx] = ClockFreqs.domain[NVAPI_GPU_PUBLIC_CLOCK_GRAPHICS].frequency;
			}
		}
		else if (NvResult == NVAPI_HANDLE_INVALIDATED)
		{
			// If the handles are not valid anymore, we cache new ones and return.
			// This query will fail but the next one should succeed.
			UE_LOGF(LogSwitchboard, Warning, "Physical GPU handles were invalidated, recaching them.");
			CacheGpuHandles();
			return;
		}
		else
		{
			UE_LOGF(LogSwitchboard, Warning, "NvAPI_GPU_GetAllClockFrequencies failed. Error code: %d", NvResult);
		}

		NV_GPU_THERMAL_SETTINGS ThermalSettings;
		ThermalSettings.version = NV_GPU_THERMAL_SETTINGS_VER;
		NvResult = NvAPI_GPU_GetThermalSettings(PhysicalGpu, NVAPI_THERMAL_TARGET_ALL, &ThermalSettings);
		if (NvResult == NVAPI_OK)
		{
			// Report max temp across all sensors for this GPU.
			for (NvU32 SensorIdx = 0; SensorIdx < ThermalSettings.count; ++SensorIdx)
			{
				const NvS32 SensorTemp = ThermalSettings.sensor[SensorIdx].currentTemp;
				if (SensorTemp > SyncStatus.GpuTemperature[PhysicalGpuIdx])
				{
					SyncStatus.GpuTemperature[PhysicalGpuIdx] = SensorTemp;
				}
			}
		}
		else if (NvResult == NVAPI_HANDLE_INVALIDATED)
		{
			// If the handles are not valid anymore, we cache new ones and return.
			// This query will fail but the next one should succeed.
			UE_LOGF(LogSwitchboard, Warning, "Physical GPU handles were invalidated, recaching them.");
			CacheGpuHandles();
			return;
		}
		else
		{
			UE_LOGF(LogSwitchboard, Warning, "NvAPI_GPU_GetThermalSettings failed. Error code: %d", NvResult);
		}
	}
}


bool FScopedNvApi::CacheGpuHandles()
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(CacheGpuHandles);

	UE::TUniqueLock<UE::FRecursiveMutex> Lock(Statics::RecursiveMutex);

	if (!IsNvApiInitialized())
	{
		return false;
	}

	Statics::PhysicalGpuHandles.Reset();

	Statics::PhysicalGpuHandles.SetNumUninitialized(NVAPI_MAX_PHYSICAL_GPUS);
	NvU32 PhysicalGpuCount;
	NvAPI_Status NvResult = NvAPI_EnumPhysicalGPUs(Statics::PhysicalGpuHandles.GetData(), &PhysicalGpuCount);

	if (NvResult != NVAPI_OK)
	{
		NvAPI_ShortString ErrorString;
		NvAPI_GetErrorMessage(NvResult, ErrorString);
		UE_LOGF(LogSwitchboard, Warning, "NvAPI_EnumPhysicalGPUs failed. Error: %ls", ANSI_TO_TCHAR(ErrorString));
		return false;
	}

	Statics::PhysicalGpuHandles.SetNum(PhysicalGpuCount);

	// Sort first by bus, then by bus slot, ascending. Consistent with task manager and others.
	Algo::Sort(Statics::PhysicalGpuHandles, [](const NvPhysicalGpuHandle& Lhs, const NvPhysicalGpuHandle& Rhs) -> bool {
		NvU32 LhsBusId, RhsBusId, LhsSlotId, RhsSlotId;
		const NvAPI_Status LhsBusResult = NvAPI_GPU_GetBusId(Lhs, &LhsBusId);
		const NvAPI_Status RhsBusResult = NvAPI_GPU_GetBusId(Rhs, &RhsBusId);
		const NvAPI_Status LhsSlotResult = NvAPI_GPU_GetBusSlotId(Lhs, &LhsSlotId);
		const NvAPI_Status RhsSlotResult = NvAPI_GPU_GetBusSlotId(Rhs, &RhsSlotId);

		if (LhsBusResult != NVAPI_OK || RhsBusResult != NVAPI_OK)
		{
			NvAPI_ShortString LhsErrorString;
			NvAPI_ShortString RhsErrorString;
			NvAPI_GetErrorMessage(LhsBusResult, LhsErrorString);
			NvAPI_GetErrorMessage(RhsBusResult, RhsErrorString);
			UE_LOGF(LogSwitchboard, Warning, "NvAPI_GPU_GetBusId failed. Errors: %ls, %ls", ANSI_TO_TCHAR(LhsErrorString), ANSI_TO_TCHAR(RhsErrorString));
			return false;
		}

		if (LhsSlotResult != NVAPI_OK || RhsSlotResult != NVAPI_OK)
		{
			NvAPI_ShortString LhsErrorString;
			NvAPI_ShortString RhsErrorString;
			NvAPI_GetErrorMessage(LhsSlotResult, LhsErrorString);
			NvAPI_GetErrorMessage(RhsSlotResult, RhsErrorString);
			UE_LOGF(LogSwitchboard, Warning, "NvAPI_GPU_GetBusSlotId failed. Errors: %ls, %ls", ANSI_TO_TCHAR(LhsErrorString), ANSI_TO_TCHAR(RhsErrorString));
			return false;
		}

		if (LhsBusId != RhsBusId)
		{
			return LhsBusId < RhsBusId;
		}

		return LhsSlotId < RhsSlotId;
	});

	return Statics::PhysicalGpuHandles.Num() > 0;
}

void FScopedNvApi::FillOutMosaicTopologies(TArray<FMosaicTopo>& MosaicTopos)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FillOutMosaicTopologies);

	UE::TUniqueLock<UE::FRecursiveMutex> Lock(Statics::RecursiveMutex);

	if (!IsNvApiInitialized())
	{
		return;
	}

	NvU32 GridCount = 0;
	TArray<NV_MOSAIC_GRID_TOPO> GridTopologies;

	// count how many grids
	{
		const NvAPI_Status Result = NvAPI_Mosaic_EnumDisplayGrids(nullptr, &GridCount);

		if (Result != NVAPI_OK)
		{
			NvAPI_ShortString ErrorString;
			NvAPI_GetErrorMessage(Result, ErrorString);
			UE_LOGF(LogSwitchboard, Warning, "NvAPI_Mosaic_EnumDisplayGrids failed. Error: %ls", ANSI_TO_TCHAR(ErrorString));
			return;
		}
	}

	// get the grids
	{
		GridTopologies.SetNumUninitialized(GridCount, EAllowShrinking::No);

		for (NvU32 TopoIdx = 0; TopoIdx < GridCount; TopoIdx++)
		{
			GridTopologies[TopoIdx].version = NV_MOSAIC_GRID_TOPO_VER;
		}

		const NvAPI_Status Result = NvAPI_Mosaic_EnumDisplayGrids(GridTopologies.GetData(), &GridCount);

		if (Result != NVAPI_OK)
		{
			NvAPI_ShortString ErrorString;
			NvAPI_GetErrorMessage(Result, ErrorString);
			UE_LOGF(LogSwitchboard, Warning, "NvAPI_Mosaic_EnumDisplayGrids failed. Error: %ls", ANSI_TO_TCHAR(ErrorString));
			return;
		}

		for (NvU32 TopoIdx = 0; TopoIdx < GridCount; TopoIdx++)
		{
			FMosaicTopo MosaicTopo;
			NV_MOSAIC_GRID_TOPO& GridTopo = GridTopologies[TopoIdx];

			MosaicTopo.Columns = GridTopo.columns;
			MosaicTopo.Rows = GridTopo.rows;
			MosaicTopo.DisplayCount = GridTopo.displayCount;

			MosaicTopo.DisplaySettings.Bpp = GridTopo.displaySettings.bpp;
			MosaicTopo.DisplaySettings.Freq = GridTopo.displaySettings.freq;
			MosaicTopo.DisplaySettings.Height = GridTopo.displaySettings.height;
			MosaicTopo.DisplaySettings.Width = GridTopo.displaySettings.width;

			MosaicTopos.Emplace(MosaicTopo);
		}
	}
}

#endif // PLATFORM_WINDOWS
