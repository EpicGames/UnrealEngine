// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if PLATFORM_WINDOWS

#include "Delegates/Delegate.h"
#include "SyncStatus.h"


/** Used to allow multi-threaded usage of NvApi and to control when it gets loaded or unloaded */
class FScopedNvApi
{
public:

	FScopedNvApi();
	~FScopedNvApi();

	/** Returns true if the NvAPI was initialized successfully */
	bool IsNvApiInitialized() const;

	/** Fills out the Sync topologies */
	void FillOutSyncTopologies(TArray<FSyncTopo>& SyncTopos) const;

	/** Fills out the sync status */
	void FillOutDriverVersion(FSyncStatus& SyncStatus) const;

	/** Fills out physical the requested gpu stats */
	void FillOutPhysicalGpuStats(FSyncStatus& SyncStatus, bool bGetUtilizations, bool bGetClocks, bool bGetTemperatures);

	/** Returns the gpu count */
	uint32 GetGpuCount();

	/** Fill out the mosaic topologies */
	void FillOutMosaicTopologies(TArray<FMosaicTopo>& MosaicTopos);

	/** Multicast delegate that gets called when a new instance of FScopedNvApi is instantiated */
	DECLARE_MULTICAST_DELEGATE(FOnNvApiInstantiated);

	// Delegate for when the index of the single tile being edited changes
	static FOnNvApiInstantiated& GetOnNvApiInstantiated();

private:

	/** Caches the gpu handles to avoid re-querying them every time gpu stats are requested */
	bool CacheGpuHandles();

private:

	/** 
	 * Private static class to encapsulate static members and avoid including nvapi.h in this header,
	 * which should reduce the temptation of using NvApi functions directly. If that happened, then
	 * the protections offered by this wrapper would not longer work.
	 */
	class Statics;

	/** True if the NvAPI was initialized for use by the current FScopedNvApi instance */
	bool bIsNvApiInitialized = false;
};

#endif // PLATFORM_WINDOWS
