// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardAuth.h"
#include "SwitchboardCredential.h"
#include "SwitchboardListenerApp.h"

#include "Containers/StaticArray.h"
#include "Containers/StringConv.h"
#include "HAL/FileManagerGeneric.h"
#include "HAL/PlatformProcess.h"
#include "IJwt.h"
#include "IPlatformCrypto.h"
#include "JsonWebToken.h"
#include "JwtAlgorithms.h"
#include "Logging/StructuredLog.h"
#include "Misc/Base64.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonWriter.h"
#include "String/BytesToHex.h"
#include "String/HexToBytes.h"

#include <iostream>
#include <string>

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include <Windows.h>
#include <ShlObj.h>
#include "Windows/HideWindowsPlatformTypes.h"
#else
#include <fcntl.h>
#include <sys/file.h>
#include <sys/stat.h>
#include <sys/types.h>
#include <termios.h>
#include <unistd.h>
#endif

#include <openssl/crypto.h>
#include <openssl/kdf.h>
#include <openssl/pem.h>
#include <openssl/rand.h>
#include <openssl/x509.h>


namespace UE::SwitchboardListener::Private
{
	constexpr FStringView PlainPasswordCredentialName = TEXTVIEW("PresharedAuthToken");
	constexpr FStringView PasswordSaltCredentialName = TEXTVIEW("PasswordSalt");
	constexpr FStringView PasswordHashCredentialName = TEXTVIEW("PasswordHash");
	constexpr FStringView LastPasswordChangeTimeCredentialName = TEXTVIEW("PasswordLastChanged");

	constexpr FStringView JwtIssuer_Value = TEXTVIEW("SwitchboardListener");

	constexpr SIZE_T ScryptNumDerivedBytes = 64;

	// The file should only be accessible for read/write by the current user.
	// If the permissions are not as expected, we need to alert the user.
	bool CheckCredentialPermissions(FStringView InFilePath)
	{
#if PLATFORM_WINDOWS
		return true;
#else
		FString NormalizedPath(InFilePath);
		FPaths::NormalizeFilename(NormalizedPath);
		NormalizedPath = FPaths::ConvertRelativePathToFull(NormalizedPath);

		// lstat is used here to explicitly not follow symbolic links.
		struct stat Stat;
		const int StatResult = lstat(TCHAR_TO_UTF8(*NormalizedPath), &Stat);
		if (StatResult == 0)
		{
			// Note: getuid is used here instead of geteuid to explicitly get the real user ID,
			// and not allow the file to be owned by another effective user ID running the program.
			const uid_t CurrentUserId = getuid();
			if (CurrentUserId != Stat.st_uid)
			{
				UE_LOGFMT(LogSwitchboard, Error,
					"The stored credential file is not owned by the current user. (Current: {CurrentUserId}; Owner: {st_uid})",
					CurrentUserId, Stat.st_uid);

				return false;
			}

			// Filter out rest of the bit flags and only check against the
			// User, Group and Other groups for their Read/Write/Execute access rights.
			const mode_t ShareFlags = Stat.st_mode & CredentialPermissionMask;
			if (ShareFlags == CredentialFilePermissionFlags)
			{
				return true;
			}
			else
			{
				UE_LOGFMT(LogSwitchboard, Error,
					"The stored credential file has insecure permissions. (Current: {ShareFlags}; Expected: {CredentialFlags})",
					ShareFlags, CredentialFilePermissionFlags);
			}
		}
		else
		{
			const int ErrorCode = errno;
			if (ErrorCode == ENOENT)
			{
				UE_LOGFMT(LogSwitchboard, Verbose, "Credential file does not exist. (Path: '{Path}')", *NormalizedPath);
			}
			else
			{
				UE_LOGFMT(LogSwitchboard, Error, "lstat failed. (errno: {ErrorCode}; Path: '{Path}')", ErrorCode, *NormalizedPath);
			}
		}

		return false;
#endif
	}

	/**
	 * On POSIX, use open + fdopen to ensure the file is created
	 * atomically with the correct, restrictive permissions.
	 */
	struct FSensitiveFile
	{
		FILE* Fp = nullptr;
#if !PLATFORM_WINDOWS
		int Fd = -1;
#endif

		~FSensitiveFile()
		{
#if PLATFORM_WINDOWS
			const bool bNeedsClose = Fp != nullptr;
#else
			const bool bNeedsClose = Fp != nullptr || Fd != -1;
#endif

			if (bNeedsClose)
			{
				ensure(Close());
			}
		}

		FILE* OpenForWrite(FStringView InFilePath)
		{
#if PLATFORM_WINDOWS
			const errno_t Result = _wfopen_s(&Fp,
				TCHAR_TO_WCHAR(InFilePath.GetData()),
				L"w");

			if (Result != 0)
			{
				UE_LOGFMT(LogSwitchboard, Error, "_wfopen_s failed ({Result})", Result);
				Fp = nullptr;
			}

			return Fp;
#else
			Fd = open(TCHAR_TO_UTF8(InFilePath.GetData()),
				O_CREAT | O_EXCL | O_WRONLY,
				CredentialFilePermissionFlags);

			if (Fd == -1)
			{
				const int OpenErrno = errno;
				UE_LOGFMT(LogSwitchboard, Error, "open failed (errno: {OpenErrno})", OpenErrno);
				return nullptr;
			}

			Fp = fdopen(Fd, "w");
			if (Fp == nullptr)
			{
				const int FdopenErrno = errno;
				UE_LOGFMT(LogSwitchboard, Error, "fdopen failed (errno: {FdopenErrno})", FdopenErrno);
				close(Fd);
				return nullptr;
			}

			return Fp;
#endif
		}

		bool Close()
		{
			bool bSuccess = true;

			if (Fp != nullptr)
			{
#if !PLATFORM_WINDOWS
				// fclose closes the underlying fd when Fp was created via fdopen
				Fd = -1;
#endif
				if (!ensure(fclose(Fp) == 0))
				{
					const int FcloseErrno = errno;
					UE_LOGFMT(LogSwitchboard, Error, "fclose failed (errno: {FcloseErrno})", FcloseErrno);
					bSuccess = false;
				}

				Fp = nullptr;
			}

#if !PLATFORM_WINDOWS
			if (Fd != -1)
			{
				if (!ensure(close(Fd) == 0))
				{
					const int CloseErrno = errno;
					UE_LOGFMT(LogSwitchboard, Error, "close failed (errno: {CloseErrno})", CloseErrno);
					bSuccess = false;
				}
				Fd = -1;
			}
#endif

			return bSuccess;
		}
	};
}


namespace UE::SwitchboardListener
{
	FString GetSwitchboardDataDirectory()
	{
#if PLATFORM_WINDOWS
		wchar_t* LocalAppDataWstr = nullptr;
		::SHGetKnownFolderPath(FOLDERID_LocalAppData, 0, nullptr, &LocalAppDataWstr);
		ON_SCOPE_EXIT{ ::CoTaskMemFree(LocalAppDataWstr); };

		const FString LocalAppDataDir(LocalAppDataWstr);
		return LocalAppDataDir / TEXT("SwitchboardListener");
#else
		const FString UserHomeDir = FPlatformProcess::UserHomeDir();
		return UserHomeDir / TEXT(".switchboard") / TEXT("listener");
#endif
	}

	FString ReadPasswordFromStdin(bool bAllowEmpty /* = false */)
	{
		FString Credential;

		// Disable echo during token input
#if PLATFORM_WINDOWS
		HANDLE Stdin = GetStdHandle(STD_INPUT_HANDLE);
		DWORD PrevConsoleMode = 0;
		GetConsoleMode(Stdin, &PrevConsoleMode);
		SetConsoleMode(Stdin, PrevConsoleMode & ~ENABLE_ECHO_INPUT);
#else
		struct termios Term;
		tcgetattr(fileno(stdin), &Term);
		Term.c_lflag &= ~ECHO;
		tcsetattr(fileno(stdin), TCSADRAIN, &Term);
#endif

		ON_SCOPE_EXIT
		{
			// Restore console echo
#if PLATFORM_WINDOWS
			SetConsoleMode(Stdin, PrevConsoleMode);
#else
			Term.c_lflag |= ECHO;
			tcsetattr(fileno(stdin), TCSADRAIN, &Term);
#endif
		};

		while (Credential.IsEmpty())
		{
			// FIXME?: Doesn't seem like we have a suitable abstraction for this?
			std::string CredFirst, CredSecond;

			printf("\nEnter password: ");
			std::getline(std::cin, CredFirst);

			if (CredFirst.empty() && !bAllowEmpty)
			{
				printf("\nEmpty password is not allowed!");
				continue;
			}

			printf("\nConfirm password: ");
			std::getline(std::cin, CredSecond);

			if (CredFirst == CredSecond)
			{
				printf("\n\n");
				Credential = UTF8_TO_TCHAR(CredFirst.c_str());
			}
			else
			{
				printf("\nProvided values didn't match!\n");
			}
		}

		return Credential;
	}

	void FillSecureRandom(TArrayView<uint8> InOutArray)
	{
		const int32 NumBytes = InOutArray.Num();
		if (ensure(NumBytes > 0))
		{
			check(1 == RAND_priv_bytes(
				reinterpret_cast<unsigned char*>(InOutArray.GetData()),
				NumBytes));
		}
	}
}


namespace UE::SwitchboardListener::Private
{
	/**
	 * Generate a 2048-bit RSA key.
	 *
	 * @param PrivateKey The private key struct
	 */
	EVP_PKEY* GeneratePrivateKey()
	{
		EVP_PKEY* PrivateKey = EVP_PKEY_new();

		// Generate the RSA key and assign it to pkey.
		RSA* Rsa = RSA_new();

		{
			BIGNUM* BigNum = BN_new();
			BN_set_word(BigNum, RSA_F4);
			RSA_generate_key_ex(Rsa, 2048, BigNum, nullptr);
			BN_free(BigNum);
		}

		// EVP_PKEY_assign_RSA will "use the supplied (Rsa) internally
		// and so (Rsa) will be freed when the parent pkey is freed."
		if (!EVP_PKEY_assign_RSA(PrivateKey, Rsa))
		{
			EVP_PKEY_free(PrivateKey);
			return nullptr;
		}

		return PrivateKey;
	}


	FString MemBIOToString(BIO* InBio)
	{
		check(InBio);

		char* Text;
		long Length = BIO_get_mem_data(InBio, &Text);

		auto Convert = StringCast<TCHAR>(Text, Length);
		FString Result(Convert.Length(), Convert.Get());
		Result.AppendChar(TEXT('\0'));
		return Result;
	}


	BIO* StringToMemBIO(const FString& InString)
	{
		return BIO_new_mem_buf(TCHAR_TO_UTF8(*InString), -1);
	}


	/**
	 * Convert a certificate to a PEM format string.
	 *
	 * @param InX509 The certificate struct.
	 */
	FString CertificateToString(X509* InX509)
	{
		check(InX509);

		BIO* Bio = BIO_new(BIO_s_mem());
		ON_SCOPE_EXIT{ BIO_free(Bio); };

		check(1 == PEM_write_bio_X509(Bio, InX509));

		return MemBIOToString(Bio);
	}


	/**
	 * Convert a certificate's public key to a PEM format string.
	 *
	 * @param InX509 The certificate struct.
	 */
	FString CertPubKeyToString(X509* InX509)
	{
		check(InX509);

		BIO* Bio = BIO_new(BIO_s_mem());
		ON_SCOPE_EXIT{ BIO_free(Bio); };

		EVP_PKEY* PubKey = X509_get0_pubkey(InX509);
		check(PubKey);

		check(1 == PEM_write_bio_PUBKEY(Bio, PubKey));

		return MemBIOToString(Bio);
	}


	bool SavePrivateKeyToFile(EVP_PKEY* InPrivateKey, const FString& InFilePath, const UTF8CHAR* InPrivateKeyPassword)
	{
		FSensitiveFile DestFile;
		FILE* DestFp = DestFile.OpenForWrite(InFilePath);
		if (!ensure(DestFp))
		{
			return false;
		}

		ON_SCOPE_EXIT{ ensure(DestFile.Close()); };

		if (!InPrivateKeyPassword || !InPrivateKeyPassword[0])
		{
			return 1 == PEM_write_PrivateKey(DestFp, InPrivateKey,
				nullptr,
				nullptr, 0, nullptr,
				nullptr);
		}
		else
		{
			return 1 == PEM_write_PrivateKey(DestFp, InPrivateKey,
				EVP_des_ede3_cbc(),
				nullptr, 0, nullptr,
				(void*)InPrivateKeyPassword);
		}
	}


	X509* LoadX509FromFile(const FString& InFilePath)
	{
#if PLATFORM_WINDOWS
		FILE* File = nullptr;
		ensure(0 == fopen_s(&File, TCHAR_TO_UTF8(*InFilePath), "r"));
#else
		FILE* File = fopen(TCHAR_TO_UTF8(*InFilePath), "r");
#endif
		ON_SCOPE_EXIT{ fclose(File); };

		return PEM_read_X509(File, nullptr, nullptr, nullptr);
	}


	EVP_PKEY* LoadPrivateKeyFromFile(const FString& InFilePath, const UTF8CHAR* InPrivateKeyPassword)
	{
		if (!CheckCredentialPermissions(InFilePath))
		{
			UE_LOGFMT(LogSwitchboard, Error, "{Func}: Insecure permissions on private key file! ({Path})",
				__FUNCTION__, InFilePath);
		}

#if PLATFORM_WINDOWS
		FILE* File = nullptr;
		ensure(0 == fopen_s(&File, TCHAR_TO_UTF8(*InFilePath), "r"));
#else
		FILE* File = fopen(TCHAR_TO_UTF8(*InFilePath), "r");
#endif
		ON_SCOPE_EXIT{ fclose(File); };

		return PEM_read_PrivateKey(File, nullptr, nullptr, (void*)InPrivateKeyPassword);
	}


	/**
	 * Generate self-signed x509 certificate.
	 *
	 * @param PrivateKey The private key struct
	 */
	X509* GenerateCertificate(EVP_PKEY* InPrivateKey)
	{
		X509* x509 = X509_new();

		// Set the serial number.
		uint64_t SerialNumber;
		check(1 == RAND_bytes(reinterpret_cast<unsigned char*>(&SerialNumber), sizeof(SerialNumber)));
		ASN1_INTEGER_set_uint64(X509_get_serialNumber(x509), SerialNumber);

		// This certificate is valid for 10 years.
		X509_gmtime_adj(X509_get_notBefore(x509), 0);
		X509_gmtime_adj(X509_get_notAfter(x509), (60L*60*24*365*10));

		// Set the public key for our certificate.
		X509_set_pubkey(x509, InPrivateKey);

		// Set the common name, and copy the subject name to the issuer name.
		const FUtf8String SubjectCommonName = FUtf8String::Printf(
			"SwitchboardListener (%s@%s)",
			TCHAR_TO_UTF8(FPlatformProcess::UserName()),
			TCHAR_TO_UTF8(FPlatformProcess::ComputerName()));

		X509_NAME* SubjectName = X509_get_subject_name(x509);
		X509_NAME_add_entry_by_txt(SubjectName, "CN", MBSTRING_UTF8,
			(const unsigned char*)*SubjectCommonName, -1, -1, 0);
		X509_set_issuer_name(x509, SubjectName);

		// Actually sign the certificate with our key.
		if (!X509_sign(x509, InPrivateKey, EVP_sha256()))
		{
			X509_free(x509);
			return nullptr;
		}

		return x509;
	}


	/** Get the default paths for (self-signed certificate, private key). */
	TTuple<FString, FString> GetDefaultSelfSignedPaths()
	{
		const TCHAR* CertFilename = TEXT("transportcert.pem");
		const TCHAR* PrivKeyFilename = TEXT("transportkey.key");

		const FString SblDataDir = GetSwitchboardDataDirectory();
		const FString CertificateFile = SblDataDir / CertFilename;
		const FString PrivateKeyFile = SblDataDir / PrivKeyFilename;

		return TTuple<FString, FString>(CertificateFile, PrivateKeyFile);
	}


	// Returns the first 128 bits of the SHA256 fingerprint in the format "09:AB:CD:..."
	FString GetX509TruncatedFingerprint(X509* InX509)
	{
		uint8 Digest[EVP_MAX_MD_SIZE];
		uint32 DigestSize = 0;
		check(1 == X509_digest(InX509, EVP_sha256(), Digest, &DigestSize));

		FString Fingerprint;
		const uint32 TruncatedDigestBytes = DigestSize / 2;
		for (uint32 ByteIdx = 0; ByteIdx < TruncatedDigestBytes; ++ByteIdx)
		{
			ByteToHex(Digest[ByteIdx], Fingerprint);
			if (ByteIdx < (TruncatedDigestBytes - 1))
			{
				Fingerprint.AppendChar(TEXT(':'));
			}
		}

		return Fingerprint;
	}


	/**
	 * Generate a private key + self-signed certificate.
	 */
	TPair<EVP_PKEY*, X509*> CreateSelfSignedPair()
	{
		EVP_PKEY* PrivateKey = GeneratePrivateKey();
		if (!ensure(PrivateKey))
		{
			return { nullptr, nullptr };
		}

		X509* Certificate = GenerateCertificate(PrivateKey);
		if (!ensure(Certificate))
		{
			EVP_PKEY_free(PrivateKey);
			return { nullptr, nullptr };
		}

		return { PrivateKey, Certificate };
	}


	bool DeriveScryptHash(FUtf8StringView InPassword, FUtf8StringView InSalt, TArrayView<uint8> OutDerivedHash)
	{
		if (!ensure(OutDerivedHash.Num() == ScryptNumDerivedBytes))
		{
			UE_LOGFMT(LogSwitchboard, Error, "{Func}: Incorrect array length (expected {ExpectedLen}, got {ArrayLen})",
				__FUNCTION__, uint64(ScryptNumDerivedBytes), OutDerivedHash.Num());
			return false;
		}

		// Set according to OWASP recommendation for minimal CPU cost.
		constexpr uint64 ScryptParam_N = 1 << 17;
		constexpr uint64 ScryptParam_r = 8;
		constexpr uint64 ScryptParam_p = 1;

		EVP_PKEY_CTX* ScryptCtx = EVP_PKEY_CTX_new_id(EVP_PKEY_SCRYPT, nullptr);
		check(ScryptCtx);
		ON_SCOPE_EXIT{ EVP_PKEY_CTX_free(ScryptCtx); };

		if (!ensure(EVP_PKEY_derive_init(ScryptCtx) > 0))
		{
			UE_LOGFMT(LogSwitchboard, Error, "{Func}: EVP_PKEY_derive_init", __FUNCTION__);
			return false;
		}

		if (!ensure(EVP_PKEY_CTX_set1_pbe_pass(ScryptCtx, InPassword.GetData(), InPassword.Len()) > 0))
		{
			UE_LOGFMT(LogSwitchboard, Error, "{Func}: EVP_PKEY_CTX_set1_pbe_pass", __FUNCTION__);
			return false;
		}

		if (!ensure(EVP_PKEY_CTX_set1_scrypt_salt(ScryptCtx, InSalt.GetData(), InSalt.Len()) > 0))
		{
			UE_LOGFMT(LogSwitchboard, Error, "{Func}: EVP_PKEY_CTX_set1_scrypt_salt", __FUNCTION__);
			return false;
		}

		if (!ensure(EVP_PKEY_CTX_set_scrypt_N(ScryptCtx, ScryptParam_N) > 0))
		{
			UE_LOGFMT(LogSwitchboard, Error, "{Func}: EVP_PKEY_CTX_set_scrypt_N", __FUNCTION__);
			return false;
		}

		if (!ensure(EVP_PKEY_CTX_set_scrypt_r(ScryptCtx, ScryptParam_r) > 0))
		{
			UE_LOGFMT(LogSwitchboard, Error, "{Func}: EVP_PKEY_CTX_set_scrypt_r", __FUNCTION__);
			return false;
		}

		if (!ensure(EVP_PKEY_CTX_set_scrypt_p(ScryptCtx, ScryptParam_p) > 0))
		{
			UE_LOGFMT(LogSwitchboard, Error, "{Func}: EVP_PKEY_CTX_set_scrypt_p", __FUNCTION__);
			return false;
		}

		SIZE_T InOutDerivedLen = ScryptNumDerivedBytes;
		if (!ensure(EVP_PKEY_derive(ScryptCtx, OutDerivedHash.GetData(), &InOutDerivedLen) > 0))
		{
			UE_LOGFMT(LogSwitchboard, Error, "{Func}: EVP_PKEY_derive", __FUNCTION__);
			return false;
		}

		if (!ensure(InOutDerivedLen == ScryptNumDerivedBytes))
		{
			UE_LOGFMT(LogSwitchboard, Error, "{Func}: Incorrect output length from EVP_PKEY_derive (Expected {ExpectedBytes}, got {ActualBytes})",
				__FUNCTION__, uint64(ScryptNumDerivedBytes), uint64(InOutDerivedLen));
			return false;
		}

		return true;
	}
}


bool FSwitchboardAuthHelper::Initialize(const FSwitchboardAuthHelper::FSettings& InSettings)
{
	using namespace UE::SwitchboardListener;
	using namespace UE::SwitchboardListener::Private;

	if (InSettings.CredentialManager)
	{
		CredentialManager = InSettings.CredentialManager;
	}
	else
	{
		CredentialManager = &ICredentialManager::GetPlatformCredentialManager();
	}

	// Mutable copy.
	FSettings Settings = InSettings;

	// At the time of writing, our Windows OpenSSL binaries appear to be built with OPENSSL_NO_SECURE_MEMORY.
#if !PLATFORM_WINDOWS
	// Try to ensure OpenSSL secure heap is initialized.
	// Not strictly required, but without it, credentials may
	// get swapped to disk, be included in core dumps, etc.
	if (!CRYPTO_secure_malloc_initialized())
	{
		const SIZE_T SecureHeapSizeBytes = 1024 * 1024;
		const SIZE_T SecureHeapMinAllocSizeBytes = 64;
		if (0 == CRYPTO_secure_malloc_init(SecureHeapSizeBytes, SecureHeapMinAllocSizeBytes))
		{
			UE_LOGFMT(LogSwitchboard, Warning, "Couldn't initialize secure heap");
		}
		else
		{
			bSecureHeapNeedsShutdown = true;
		}
	}
#endif // #if !PLATFORM_WINDOWS

	// If the user didn't specify key pair, check the default path,
	// and possibly generate a new self-signed key pair.
	const bool bUserSpecifiedKeyPair = Settings.Certificate && Settings.PrivateKey;
	if (!bUserSpecifiedKeyPair)
	{
		if (Settings.Certificate || Settings.PrivateKey)
		{
			UE_LOGFMT(LogSwitchboard, Error, "{Func}: Must specify both Certificate AND PrivateKey, or neither; ignoring", __FUNCTION__);
		}

		if (Settings.PrivateKeyPassword)
		{
			UE_LOGFMT(LogSwitchboard, Error, "{Func}: Cannot specify password for auto-generated private key; ignoring", __FUNCTION__);
			Settings.PrivateKeyPassword.Reset();
		}

		CheckOrCreateSelfSignedKeyPair(Settings);
	}

	if (!Settings.PrivateKey)
	{
		UE_LOGFMT(LogSwitchboard, Error, "{Func}: No private key specified", __FUNCTION__);
		return false;
	}
	else if (const FString* PrivateKeyPathStr = Settings.PrivateKey->TryGet<FString>())
	{
		PrivateKeyFilePath = **PrivateKeyPathStr;
		UE_LOGFMT(LogSwitchboard, Verbose, "{Func}: Loading private key from file {PrivateKeyPath}",
			__FUNCTION__, *PrivateKeyPathStr);
		PrivateKey = LoadPrivateKeyFromFile(*PrivateKeyPathStr, PrivateKeyPassword);
	}
	else
	{
		UE_LOGFMT(LogSwitchboard, Verbose, "{Func}: Using provided EVP_PKEY object", __FUNCTION__);
		check(Settings.PrivateKey->IsType<EVP_PKEY*>());
		PrivateKey = Settings.PrivateKey->Get<EVP_PKEY*>();
	}

	if (!PrivateKey)
	{
		UE_LOGFMT(LogSwitchboard, Error, "{Func}: Failed to load private key", __FUNCTION__);
		return false;
	}

	if (!Settings.Certificate)
	{
		UE_LOGFMT(LogSwitchboard, Error, "{Func}: No certificate specified", __FUNCTION__);
		return false;
	}
	else if (const FString* CertificatePathStr = Settings.Certificate->TryGet<FString>())
	{
		CertificateFilePath = **CertificatePathStr;
		UE_LOGFMT(LogSwitchboard, Verbose, "{Func}: Loading certificate from file {CertificatePath}",
			__FUNCTION__, *CertificatePathStr);
		Certificate = LoadX509FromFile(*CertificatePathStr);
	}
	else
	{
		UE_LOGFMT(LogSwitchboard, Verbose, "{Func}: Using provided X509 object", __FUNCTION__);
		check(Settings.Certificate->IsType<X509*>());
		Certificate = Settings.Certificate->Get<X509*>();
	}

	if (!Certificate)
	{
		UE_LOGFMT(LogSwitchboard, Error, "{Func}: Failed to load certificate", __FUNCTION__);
		return false;
	}

	// Finally, load the auth password (or its hash)
	if (CredentialManager->IsEncryptedAtRest())
	{
		// This salt is ephemeral since we store the plain text password.
		HashSalt = GenerateHashSalt();

		if (TSharedPtr<ICredential> SavedPassword = CredentialManager->LoadCredential(PlainPasswordCredentialName))
		{
			UE_LOGFMT(LogSwitchboard, Display, "Using stored password");

			const bool bExpireExistingTokens_false = false;
			ensure(SetAuthPassword(SavedPassword->GetBlob(), bExpireExistingTokens_false));
		}
	}
	else
	{
		if (TSharedPtr<ICredential> SavedSalt = CredentialManager->LoadCredential(PasswordSaltCredentialName))
		{
			UE_LOGFMT(LogSwitchboard, Display, "Using stored password salt");
			HashSalt = FUtf8String(SavedSalt->GetBlob());
		}
		else
		{
			HashSalt = GenerateHashSalt();

			// Persist the salt since we'll persist the hash.
			ensure(CredentialManager->SaveCredential(
				PasswordSaltCredentialName,
				FStringView(),
				UTF8_TO_TCHAR(*HashSalt)));
		}

		if (TSharedPtr<ICredential> SavedHash = CredentialManager->LoadCredential(PasswordHashCredentialName))
		{
			const int32 ExpectedHashStrLen = 2 * ScryptNumDerivedBytes;

			if (ensure(SavedHash->GetBlob().Len() == ExpectedHashStrLen))
			{
				UE_LOGFMT(LogSwitchboard, Display, "Using stored password hash");
				HashedAuthPassword.SetNumUninitialized(ScryptNumDerivedBytes);
				UE::String::HexToBytes(SavedHash->GetBlob(), HashedAuthPassword.GetData());
			}
			else
			{
				UE_LOGFMT(LogSwitchboard, Error, "Stored password hash has wrong length (expected {ExpectedLen}, got {ActualLen}); discarding",
					ExpectedHashStrLen, SavedHash->GetBlob().Len());
			}
		}
	}

	// We also need to know if we should ignore JWTs issued prior to our last password change.
	if (TSharedPtr<ICredential> SavedExpiry = CredentialManager->LoadCredential(LastPasswordChangeTimeCredentialName))
	{
		LexFromString(LastPasswordChangeTimeUtc, *FString(SavedExpiry->GetBlob()));

		const FDateTime ExpiryTime = FDateTime::FromUnixTimestamp(LastPasswordChangeTimeUtc);
		UE_LOGFMT(LogSwitchboard, Verbose, "Ignoring tokens issued prior to {ExpiryTime} UTC", ExpiryTime);
	}

	return true;
}


bool FSwitchboardAuthHelper::CheckOrCreateSelfSignedKeyPair(FSettings& InOutSettings)
{
	using namespace UE::SwitchboardListener;
	using namespace UE::SwitchboardListener::Private;

	const TTuple<FString, FString> DefaultCertKeyPaths = GetDefaultSelfSignedPaths();
	const FString& CertificatePathStr = DefaultCertKeyPaths.Get<0>();
	const FString& PrivateKeyPathStr = DefaultCertKeyPaths.Get<1>();
	InOutSettings.Certificate.Emplace(TInPlaceType<FString>(), CertificatePathStr);
	InOutSettings.PrivateKey.Emplace(TInPlaceType<FString>(), PrivateKeyPathStr);

	bool bCanLoadKeyPair =
		FPaths::FileExists(CertificatePathStr) &&
		FPaths::FileExists(PrivateKeyPathStr);

	// Platforms with a suitable credential store encrypt the generated
	// private key with a random string, and persist it for the user.
	if (CredentialManager->IsEncryptedAtRest())
	{
		// Try to load the private key password.
		const FString PrivateKeyPwCredentialName =
			FString::Printf(TEXT("PrivateKeyPassword_%s"), *PrivateKeyPathStr);

		if (TSharedPtr<ICredential> PrivateKeyPwCredential =
			CredentialManager->LoadCredential(PrivateKeyPwCredentialName))
		{
			FStringView BlobView = PrivateKeyPwCredential->GetBlob();
			auto BlobU8 = StringCast<UTF8CHAR>(BlobView.GetData(), BlobView.Len());
			PrivateKeyPassword = (UTF8CHAR*)OPENSSL_secure_zalloc(BlobU8.Length() + 1);
			FMemory::Memcpy(PrivateKeyPassword, BlobU8.Get(), BlobU8.Length());
		}
		else
		{
			if (bCanLoadKeyPair)
			{
				UE_LOGFMT(LogSwitchboard, Error, "Unable to load private key password; a new key pair will be generated");
				bCanLoadKeyPair = false;
			}

			UE_LOGFMT(LogSwitchboard, Verbose, "Generating new random private key password");

			constexpr int32 RandomPwByteLength = 20;

			uint8* RandomPwByteBuf = (uint8*)OPENSSL_secure_malloc(RandomPwByteLength);
			ON_SCOPE_EXIT{ OPENSSL_secure_clear_free(RandomPwByteBuf, RandomPwByteLength); };

			TArrayView<uint8> RandomPwBytes(RandomPwByteBuf, RandomPwByteLength);
			FillSecureRandom(RandomPwBytes);

			PrivateKeyPassword = (UTF8CHAR*)OPENSSL_secure_zalloc((RandomPwByteLength * 2) + 1);
			UE::String::BytesToHex(RandomPwBytes, PrivateKeyPassword);

			ensure(CredentialManager->SaveCredential(
				PrivateKeyPwCredentialName,
				FStringView(),
				UTF8_TO_TCHAR(PrivateKeyPassword)));
		}
	}

	if (!bCanLoadKeyPair)
	{
		UE_LOGFMT(LogSwitchboard, Display, "SwitchboardListener requires a TLS certificate to be specified; generating self-signed certificate...");

		Tie(PrivateKey, Certificate) = CreateSelfSignedPair();

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		PlatformFile.CreateDirectoryTree(*GetSwitchboardDataDirectory());

		if (!FFileHelper::SaveStringToFile(CertificateToString(Certificate), *CertificatePathStr))
		{
			UE_LOGFMT(LogSwitchboard, Fatal, "{Func}: Failed to save certificate to {CertPath}",
				__FUNCTION__, CertificatePathStr);

			return false;
		}

		if (!SavePrivateKeyToFile(PrivateKey, PrivateKeyPathStr, PrivateKeyPassword))
		{
			UE_LOGFMT(LogSwitchboard, Fatal, "{Func}: Failed to save private key to {KeyPath}",
				__FUNCTION__, PrivateKeyPathStr);

			return false;
		}

		if (!CheckCredentialPermissions(PrivateKeyPathStr))
		{
			UE_LOGFMT(LogSwitchboard, Error, "{Func}: Insecure permissions on private key file! ({Path})",
				__FUNCTION__, PrivateKeyPathStr);
		}

		UE_LOGFMT(LogSwitchboard, Display, "Your new self-signed certificate fingerprint is: {Fingerprint}",
			*GetX509TruncatedFingerprint(Certificate));
	}

	return true;
}


FUtf8String FSwitchboardAuthHelper::GenerateHashSalt()
{
	using namespace UE::SwitchboardListener;

	UE_LOGFMT(LogSwitchboard, Verbose, "Generating new password hash salt");

	constexpr int32 SaltByteLength = 4;
	TStaticArray<uint8, SaltByteLength> SaltBytes;
	FillSecureRandom(SaltBytes);

	constexpr int32 SaltStrLength = SaltByteLength * 2 + 1;
	TUtf8StringBuilder<SaltStrLength> SaltStrBuilder;
	UE::String::BytesToHex(SaltBytes, SaltStrBuilder);

	return SaltStrBuilder.ToString();
}


bool FSwitchboardAuthHelper::UpdateLastPasswordChangeTimeUtc(
	FDateTime InChangeTimeUtc /* = FDateTime::UtcNow() */
)
{
	using namespace UE::SwitchboardListener::Private;

	LastPasswordChangeTimeUtc = InChangeTimeUtc.ToUnixTimestamp();
	return CredentialManager->SaveCredential(LastPasswordChangeTimeCredentialName,
		FStringView(),
		LexToString(LastPasswordChangeTimeUtc)
	);
}


void FSwitchboardAuthHelper::Shutdown()
{
	if (PrivateKey)
	{
		EVP_PKEY_free(PrivateKey);
		PrivateKey = nullptr;
	}

	if (Certificate)
	{
		X509_free(Certificate);
		Certificate = nullptr;
	}

	if (PrivateKeyPassword)
	{
		OPENSSL_secure_clear_free(PrivateKeyPassword, FCStringUtf8::Strlen(PrivateKeyPassword) + 1);
		PrivateKeyPassword = nullptr;
	}

	if (PlainAuthPassword)
	{
		OPENSSL_secure_clear_free(PlainAuthPassword, FCStringUtf8::Strlen(PlainAuthPassword));
		PlainAuthPassword = nullptr;
	}

	if (bSecureHeapNeedsShutdown)
	{
		CRYPTO_secure_malloc_done();
		bSecureHeapNeedsShutdown = false;
	}
}


const FUtf8String& FSwitchboardAuthHelper::GetCertificateFilePath() const
{
	return CertificateFilePath;
}


const FUtf8String& FSwitchboardAuthHelper::GetPrivateKeyFilePath() const
{
	return PrivateKeyFilePath;
}


const UTF8CHAR* FSwitchboardAuthHelper::GetPrivateKeyPassword() const
{
	return PrivateKeyPassword;
}


bool FSwitchboardAuthHelper::IsAuthPasswordSet() const
{
	return HashedAuthPassword.Num() > 0;
}


const UTF8CHAR* FSwitchboardAuthHelper::GetAuthPassword() const
{
	return PlainAuthPassword;
}




bool FSwitchboardAuthHelper::SetAuthPassword(
	FStringView InNewPassword,
	bool bExpireExistingTokens /* = true */
)
{
	using namespace UE::SwitchboardListener;
	using namespace UE::SwitchboardListener::Private;

	if (!ensure(!InNewPassword.IsEmpty()))
	{
		return false;
	}

	auto InNewPasswordU8 = StringCast<UTF8CHAR>(InNewPassword.GetData(), InNewPassword.Len());
	HashedAuthPassword.SetNumUninitialized(ScryptNumDerivedBytes);
	check(DeriveScryptHash(InNewPasswordU8, HashSalt, HashedAuthPassword));

	if (bExpireExistingTokens)
	{
		ensure(UpdateLastPasswordChangeTimeUtc());
	}

	if (CredentialManager->IsEncryptedAtRest())
	{
		if (PlainAuthPassword)
		{
			OPENSSL_secure_clear_free(PlainAuthPassword, FCStringUtf8::Strlen(PlainAuthPassword));
		}

		PlainAuthPassword = (UTF8CHAR*)OPENSSL_secure_zalloc(InNewPasswordU8.Length() + 1);
		FMemory::Memcpy(PlainAuthPassword, InNewPasswordU8.Get(), InNewPasswordU8.Length());

		UE_LOGFMT(LogSwitchboard, Display, "Storing provided password");
		ensure(CredentialManager->SaveCredential(
			PlainPasswordCredentialName,
			FStringView(),
			InNewPassword));
	}
	else
	{
		constexpr int32 HashStrLength = ScryptNumDerivedBytes * 2 + 1;
		TStringBuilder<HashStrLength> SaltStrBuilder;
		UE::String::BytesToHex(HashedAuthPassword, SaltStrBuilder);

		UE_LOGFMT(LogSwitchboard, Display, "Storing hash of provided password");
		ensure(CredentialManager->SaveCredential(
			PasswordHashCredentialName,
			FStringView(),
			SaltStrBuilder.ToString()));
	}

	return true;
}


bool FSwitchboardAuthHelper::ValidatePassword(FStringView InAttempt)
{
	using namespace UE::SwitchboardListener::Private;

	auto AttemptU8 = StringCast<UTF8CHAR>(InAttempt.GetData(), InAttempt.Len());

	TArray<uint8> HashedAttempt;
	HashedAttempt.SetNumUninitialized(ScryptNumDerivedBytes);
	check(DeriveScryptHash(AttemptU8, HashSalt, HashedAttempt));

	check(HashedAttempt.Num() == HashedAuthPassword.Num());

	return 0 == CRYPTO_memcmp(HashedAttempt.GetData(), HashedAuthPassword.GetData(), HashedAuthPassword.Num());
}


namespace UE::SwitchboardListener::Private
{
	FString MakeJWTHeader(FStringView InAlgorithm)
	{
		FString Header;
		TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter =
			TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Header);

		JsonWriter->WriteObjectStart();
		JsonWriter->WriteValue(TEXTVIEW("alg"), InAlgorithm);
		JsonWriter->WriteValue(TEXTVIEW("typ"), TEXTVIEW("JWT"));
		JsonWriter->WriteObjectEnd();
		ensure(JsonWriter->Close());

		return Header;
	}

	FString MakeJWTPayload(const TMap<FStringView, FStringView>& InPayloadFields)
	{
		FString Payload;
		TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter =
			TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Payload);

		JsonWriter->WriteObjectStart();
		for (const TPair<FStringView, FStringView>& Field : InPayloadFields)
		{
			JsonWriter->WriteValue(Field.Key, Field.Value);
		}
		JsonWriter->WriteObjectEnd();
		ensure(JsonWriter->Close());

		return Payload;
	}
}


FString FSwitchboardAuthHelper::IssueJWT(const TMap<FStringView, FStringView>& InPayloadFields)
{
	using namespace UE::SwitchboardListener::Private;

	// Make header
	constexpr FStringView Header_Alg = TEXTVIEW("RS256");
	const FString HeaderJson = MakeJWTHeader(Header_Alg);

	// Make payload
	TMap<FStringView, FStringView> PayloadFields = InPayloadFields;

	constexpr FStringView Issuer_Field = TEXTVIEW("iss");
	PayloadFields.Add(Issuer_Field, JwtIssuer_Value);

	// Issued at the current time.
	constexpr FStringView IssuedAt_Field = TEXTVIEW("iat");
	const FDateTime IssuedAtTime = FDateTime::UtcNow();
	const FString IssuedAtTimeStr = LexToString(IssuedAtTime.ToUnixTimestamp());
	PayloadFields.Add(IssuedAt_Field, IssuedAtTimeStr);

	// Expires in 10 years.
	constexpr FStringView Expiration_Field = TEXTVIEW("exp");
	const FDateTime ExpirationTime = IssuedAtTime + FTimespan::FromDays(365 * 10);
	const FString ExpirationTimeStr = LexToString(ExpirationTime.ToUnixTimestamp());
	PayloadFields.Add(Expiration_Field, ExpirationTimeStr);

	const FString PayloadJson = MakeJWTPayload(PayloadFields);

	// Encode and concatenate header + payload, giving us the message to be signed.
	const FString HeaderB64 = FBase64::Encode(HeaderJson, EBase64Mode::UrlSafe).Replace(TEXT("="), TEXT(""));
	const FString PayloadB64 = FBase64::Encode(PayloadJson, EBase64Mode::UrlSafe).Replace(TEXT("="), TEXT(""));
	const FString Message = HeaderB64 + TEXT('.') + PayloadB64;

	// Take a UTF8 view of the message for computing the signature.
	auto MessageU8 = StringCast<UTF8CHAR>(*Message);
	TArrayView<const uint8> MessageByteView((const uint8*)MessageU8.Get(), MessageU8.Length());

	// Get the underlying RSA private key and cast it to the opaque handle type.
	check(PrivateKey);
	RSA* PrivateKeyRsa = EVP_PKEY_get0_RSA(PrivateKey);
	check(PrivateKeyRsa);

	FRSAKeyHandle KeyHandle = reinterpret_cast<FRSAKeyHandle>(PrivateKeyRsa);

	// Compute the signed digest on the message hash.
	TUniquePtr<FEncryptionContext> EncryptionContext = IPlatformCrypto::Get().CreateContext();
	check(EncryptionContext);

	TArray<uint8> HashedMessage;
	check(EncryptionContext->CalcSHA256(MessageByteView, HashedMessage));

	TArray<uint8> Signature;
	check(EncryptionContext->DigestSign_RS256(HashedMessage, Signature, KeyHandle));

	const FString SignatureB64 = FBase64::Encode(Signature, EBase64Mode::UrlSafe).Replace(TEXT("="), TEXT(""));
	return Message + TEXT('.') + SignatureB64;
}


TOptional<FJsonWebToken> FSwitchboardAuthHelper::GetValidatedJWT(FStringView InEncodedJwt) const
{
	using namespace UE::SwitchboardListener::Private;

	FJwtAlgorithm_RS256 Verifier;
	if (!ensure(Verifier.SetPublicKey(CertPubKeyToString(Certificate))))
	{
		UE_LOGFMT(LogSwitchboard, Error, "Public key error during JWT validation");
		return {};
	}

	TOptional<FJsonWebToken> Jwt = UE::JWT::FromString(InEncodedJwt);
	if (!ensure(Jwt))
	{
		UE_LOGFMT(LogSwitchboard, Error, "Error parsing JWT");
		return {};
	}

	if (!ensure(Jwt->Verify(Verifier, JwtIssuer_Value)))
	{
		UE_LOGFMT(LogSwitchboard, Error, "JWT validation failed");
		return {};
	}

	// Check to see if we've implicitly expired the JWT on password change.
	int64 IssuedAt;
	if (!ensure(Jwt->GetIssuedAt(IssuedAt)))
	{
		UE_LOGFMT(LogSwitchboard, Error, "Couldn't get JWT issued at time");
		return {};
	}

	if (IssuedAt < LastPasswordChangeTimeUtc)
	{
		UE_LOGFMT(LogSwitchboard, Error, "JWT issued at time predates last password change");
		return {};
	}

	return Jwt;
}


bool FSwitchboardAuthHelper::IsValidJWT(FStringView InEncodedJwt) const
{
	return GetValidatedJWT(InEncodedJwt).IsSet();
}
