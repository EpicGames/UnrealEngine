// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardListener.h"

#include "CpuUtilizationMonitor.h"
#include "GpuClockerManager.h"
#include "SwitchboardAuth.h"
#include "SwitchboardListenerApp.h"
#include "SwitchboardMessageFuture.h"
#include "SwitchboardPacket.h"
#include "SwitchboardProcess.h"
#include "SwitchboardProtocol.h"
#include "SwitchboardTasks.h"
#include "SyncStatus.h"

#include "Algo/Find.h"
#include "Async/Async.h"
#include "Async/AsyncWork.h"
#include "Common/TcpListener.h"
#include "Dom/JsonObject.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformTime.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "IPAddress.h"
#include "Logging/StructuredLog.h"
#include "Misc/Base64.h"
#include "Misc/CommandLine.h"
#include "Misc/Compression.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

#include <atomic>

#if PLATFORM_WINDOWS

#pragma warning(push)
#pragma warning(disable : 4005)	// Disable macro redefinition warning for compatibility with Windows SDK 8+

#include "Windows/AllowWindowsPlatformTypes.h"
#include <evntrace.h>
#include <shellapi.h>
#include <tlhelp32.h>
#include <Windows.h>
#include <WinUser.h>
#include "Windows/HideWindowsPlatformTypes.h"

#include "ScopedNvApi.h"

#pragma warning(pop)

#endif


#define QUIC_ENSURE(X) ensure(QUIC_SUCCEEDED(X))


const FIPv4Endpoint FSwitchboardListener::InvalidEndpoint(FIPv4Address::LanBroadcast, 0);


namespace
{
#if !UE_BUILD_DEBUG
	const double DefaultInactiveTimeoutSeconds = 5.0;
#else
	// Infinite timeout in debug; convenient for pausing at breakpoints for long periods.
	const double DefaultInactiveTimeoutSeconds = 0.0;
#endif

	bool TryFindIdInBrokenMessage(const FString& InMessage, FGuid& OutGuid)
	{
		// Try to parse message that could not be parsed regularly and look for the message ID.
		// This way we can at least tell Switchboard which message was broken.
		const int32 IdIdx = InMessage.Find(TEXT("\"id\""));
		if (IdIdx > 0)
		{
			const FString Chopped = InMessage.RightChop(IdIdx);
			FString LeftOfComma;
			FString RightOfComma;
			if (Chopped.Split(",", &LeftOfComma, &RightOfComma))
			{
				FString LeftOfColon;
				FString RightOfColon;
				if (LeftOfComma.Split(":", &LeftOfColon, &RightOfColon))
				{
					RightOfColon.TrimStartAndEndInline();
					bool bDoubleQuotesRemoved = false;
					RightOfColon.TrimQuotesInline(&bDoubleQuotesRemoved);
					if (!bDoubleQuotesRemoved)
					{
						// remove single quotes if there were no double quotes
						RightOfColon.LeftChopInline(1);
						RightOfColon.RightChopInline(1);
					}

					return FGuid::Parse(RightOfColon, OutGuid);
				}
			}
		}

		return false;
	}

	QUIC_ADDR QuicAddrFromEndpoint(FIPv4Endpoint Endpoint)
	{
		QUIC_ADDR QuicAddr = {};

		inet_pton(AF_INET, TCHAR_TO_ANSI(*Endpoint.Address.ToString()),
			&QuicAddr.Ipv4.sin_addr.s_addr);

		QuicAddrSetFamily(&QuicAddr, QUIC_ADDRESS_FAMILY_INET);
		QuicAddrSetPort(&QuicAddr, Endpoint.Port);

		return QuicAddr;
	}


	FIPv4Endpoint EndpointFromQuicAddr(QUIC_ADDR QuicAddr)
	{
		FIPv4Endpoint Endpoint;

		char Ipv4Buf[INET_ADDRSTRLEN];
		inet_ntop(AF_INET, &QuicAddr.Ipv4.sin_addr.s_addr, Ipv4Buf, sizeof(Ipv4Buf));
		FIPv4Address::Parse(ANSI_TO_TCHAR(Ipv4Buf), Endpoint.Address);

		Endpoint.Port = QuicAddrGetPort(&QuicAddr);
		return Endpoint;
	}

	FString GetQuicErrorStr(QUIC_STATUS InStatus)
	{
		static TMap<QUIC_STATUS, FString> StatStringMap = {
			{QUIC_STATUS_SUCCESS			 , TEXT("The operation completed successfully.")},
			{QUIC_STATUS_PENDING			 , TEXT("The operation is pending.")},
			{QUIC_STATUS_CONTINUE			 , TEXT("The operation will continue.")},
			{QUIC_STATUS_OUT_OF_MEMORY		 , TEXT("Allocation of memory failed.")},
			{QUIC_STATUS_INVALID_PARAMETER	 , TEXT("An invalid parameter was encountered.")},
			{QUIC_STATUS_INVALID_STATE		 , TEXT("The current state was not valid for this operation.")},
			{QUIC_STATUS_NOT_SUPPORTED	     , TEXT("The operation was not supported.")},
			{QUIC_STATUS_NOT_FOUND	         , TEXT("The object was not found.")},
			{QUIC_STATUS_BUFFER_TOO_SMALL	 , TEXT("The buffer was too small for the operation.")},
			{QUIC_STATUS_HANDSHAKE_FAILURE	 , TEXT("The connection handshake failed.")},
			{QUIC_STATUS_ABORTED			 , TEXT("The connection or stream was aborted.")},
			{QUIC_STATUS_ADDRESS_IN_USE		 , TEXT("The local address is already in use.")},
			{QUIC_STATUS_INVALID_ADDRESS	 , TEXT("Binding to socket failed, likely caused by a family mismatch between local and remote address.")},
			{QUIC_STATUS_CONNECTION_TIMEOUT	 , TEXT("The connection timed out waiting for a response from the peer.")},
			{QUIC_STATUS_CONNECTION_IDLE	 , TEXT("The connection timed out from inactivity.")},
			{QUIC_STATUS_INTERNAL_ERROR		 , TEXT("An internal error was encountered.")},
			{QUIC_STATUS_UNREACHABLE		 , TEXT("The server is currently unreachable.")},
			{QUIC_STATUS_CONNECTION_REFUSED	 , TEXT("The server refused the connection.")},
			{QUIC_STATUS_PROTOCOL_ERROR		 , TEXT("A protocol error was encountered.")},
			{QUIC_STATUS_VER_NEG_ERROR		 , TEXT("A version negotiation error was encountered.")},
			{QUIC_STATUS_USER_CANCELED		 , TEXT("The peer app/user canceled the connection during the handshake.")},
			{QUIC_STATUS_ALPN_NEG_FAILURE	 , TEXT("The connection handshake failed to negotiate a common ALPN.")},
			{QUIC_STATUS_STREAM_LIMIT_REACHED, TEXT("A stream failed to start because the peer doesn't allow any more to be open at this time.")}
		};
		if (FString* ErrorString = StatStringMap.Find(InStatus))
		{
			return *ErrorString;
		}
		return TEXT("Unknown MsQuic error.");
	}

#if PLATFORM_WINDOWS
	/** 
	 * Manages an instance of FScopedNvApi to keep the library initialized until it doesn't get used for a while.
	 * The objective is to work around an issue where NvApi can sometimes cause hitches after it has been initialized for 
	 * around 12 hours. If the client is disconnected or is not requesting data that requires NvApi, then this
	 * object will not keep an instance of FScopedNvApi so that the libary can be uninitialized. But if it is being
	 * actively used, then it will keep this object alive and hence the library initialized for normal usage.
	 */
	class FWorkaroundForHitchingAfterHours
	{
	private:

		FWorkaroundForHitchingAfterHours()
		{
			FScopedNvApi::GetOnNvApiInstantiated().AddRaw(this, &FWorkaroundForHitchingAfterHours::OnNvApiInstantiated);
		}

		~FWorkaroundForHitchingAfterHours()
		{
			FScopedNvApi::GetOnNvApiInstantiated().RemoveAll(this);
		}

		/** Called when a new insance of FScopedNvApi is created. This will signal that it is being used elsewhere. */
		void OnNvApiInstantiated()
		{
			LastTimeInstantiatedSeconds = FPlatformTime::Seconds();
			bNvApiInstanceNeeded = true;
		}

	public:

		/** Singleton getter. We only want one instance of this workaround class. */
		static FWorkaroundForHitchingAfterHours& Get()
		{
			static FWorkaroundForHitchingAfterHours Instance;
			return Instance;
		}

		/** Call this periodically in the main thread to update the internal instance of FScopedNvApi */
		void Tick()
		{
			if (bNvApiInstanceNeeded)
			{
				if (!ScopedNvApi.IsValid())
				{
					ScopedNvApi = MakeShared<FScopedNvApi>();
				}

				bNvApiInstanceNeeded = false;
			}
			else if (ScopedNvApi.IsValid())
			{
				const double NowSeconds = FPlatformTime::Seconds();
				const double ElapsedSeconds = NowSeconds - LastTimeInstantiatedSeconds;

				if (ElapsedSeconds > NvApiUsageTimeoutMinutes * 60)
				{
					ScopedNvApi.Reset();
				}
			}
		}

	private:

		/** No new instancing of FScopedNvApi for this time will trigger a reset of the local instance kept by this object */
		const double NvApiUsageTimeoutMinutes = 10;

		/** Signals a request of creating a new instance of FScopedNvApi */
		std::atomic<bool> bNvApiInstanceNeeded = false;

		/** A record of the last time that an instance of FScopedNvApi was created. Use to detect usage timeouts. */
		std::atomic<double> LastTimeInstantiatedSeconds = 0;

		/** The instance of ScopedNvApi that will be kept alive until a timout of lack of usage of the library */
		TSharedPtr<FScopedNvApi> ScopedNvApi;
	};
#endif // PLATFORM_WINDOWS
}

struct FRunningProcess
{
	uint32 PID = 0;
	FGuid UUID;
	FProcHandle Handle;

	void* StdoutParentReadPipe = nullptr;
	void* StdoutChildWritePipe = nullptr;
	void* StderrParentReadPipe = nullptr;
	void* StderrChildWritePipe = nullptr;
	void* StdinChildReadPipe = nullptr;
	void* StdinParentWritePipe = nullptr;

	TArray<uint8> Stdout;
	TArray<uint8> Stderr;

	FIPv4Endpoint Recipient;
	FString Path;
	FString Name;
	FString Caller;

	std::atomic<bool> bPendingKill = false;
	bool bUpdateClientsWithStdout = false;
	bool bCombinedStdoutStderr = true;
	bool bLockGpuClock = false;

	FTSSimpleMulticastDelegate OnProcessEnded;

	bool CreatePipes()
	{
		if (!FPlatformProcess::CreatePipe(StdoutParentReadPipe, StdoutChildWritePipe))
		{
			UE_LOGF(LogSwitchboard, Error, "Could not create pipe to read process output!");
			return false;
		}

		if (!bCombinedStdoutStderr)
		{
			if (!FPlatformProcess::CreatePipe(StderrParentReadPipe, StderrChildWritePipe))
			{
				UE_LOGF(LogSwitchboard, Error, "Could not create pipe to read process stderr!");
				return false;
			}
		}

		if (!FPlatformProcess::CreatePipe(StdinChildReadPipe, StdinParentWritePipe, true))
		{
			UE_LOGF(LogSwitchboard, Error, "Could not create pipe to write process input!");
			return false;
		}

		return true;
	}

	void ClosePipes()
	{
		FPlatformProcess::ClosePipe(StdoutParentReadPipe, StdoutChildWritePipe);
		FPlatformProcess::ClosePipe(StderrParentReadPipe, StderrChildWritePipe);
		FPlatformProcess::ClosePipe(StdinChildReadPipe, StdinParentWritePipe);
	}
};

FSwitchboardCommandLineOptions FSwitchboardCommandLineOptions::FromString(const TCHAR* CommandLine)
{
	FSwitchboardCommandLineOptions OutOptions;

	TArray<FString> Tokens;
	TArray<FString> Switches;
	FCommandLine::Parse(CommandLine, Tokens, Switches);
	TMap<FString, FString> SwitchPairs;
	for (int32 SwitchIdx = Switches.Num() - 1; SwitchIdx >= 0; --SwitchIdx)
	{
		FString& Switch = Switches[SwitchIdx];
		TArray<FString> SplitSwitch;
		if (2 == Switch.ParseIntoArray(SplitSwitch, TEXT("="), true))
		{
			SwitchPairs.Add(SplitSwitch[0], SplitSwitch[1].TrimQuotes());
			Switches.RemoveAt(SwitchIdx);
		}
	}

	if (Switches.Contains(TEXT("version")))
	{
		OutOptions.OutputVersion = true;
	}

	if (Switches.Contains(TEXT("noMinimizeOnLaunch")))
	{
		OutOptions.MinimizeOnLaunch = false;
	}

	if (SwitchPairs.Contains(TEXT("ip")))
	{
		FIPv4Address ParseAddr;
		if (FIPv4Address::Parse(SwitchPairs[TEXT("ip")], ParseAddr))
		{
			OutOptions.Address = ParseAddr;
		}
	}

	if (SwitchPairs.Contains(TEXT("port")))
	{
		OutOptions.Port = FCString::Atoi(*SwitchPairs[TEXT("port")]);
	}

	if (SwitchPairs.Contains(TEXT("RedeployFromPid")))
	{
		OutOptions.RedeployFromPid = FCString::Atoi64(*SwitchPairs[TEXT("RedeployFromPid")]);
	}

	return OutOptions;
}

FString FSwitchboardCommandLineOptions::ToString(bool bIncludeRedeploy /* = false */) const
{
	TArray<FString> Args;

	if (!MinimizeOnLaunch)
	{
		Args.Add(TEXT("-noMinimizeOnLaunch"));
	}

	if (Address.IsSet())
	{
		Args.Add(FString::Printf(TEXT("-ip=%s"), *Address.GetValue().ToString()));
	}

	if (Port.IsSet())
	{
		Args.Add(FString::Printf(TEXT("-port=%u"), Port.GetValue()));
	}

	if (bIncludeRedeploy)
	{
		if (RedeployFromPid.IsSet())
		{
			Args.Add(FString::Printf(TEXT("-RedeployFromPid=%u"), RedeployFromPid.GetValue()));
		}
	}

	return FString::Join(Args, TEXT(" "));
}


FSwitchboardListener::FSwitchboardListener(const FSwitchboardCommandLineOptions& InOptions)
	: Options(InOptions)
	, CpuMonitor(MakeShared<FCpuUtilizationMonitor>())
	, GpuClockerManager(MakeShared<FGpuClockerManager>())
	, CachedMosaicToposLock(MakeShared<FRWLock>())
	, CachedMosaicTopos(MakeShared<TArray<FMosaicTopo>>())
{
#if PLATFORM_WINDOWS
	// Cache Mosaic Topologies
	FScopedNvApi NvApi;
	NvApi.FillOutMosaicTopologies(*CachedMosaicTopos);
#endif // PLATFORM_WINDOWS

	const int32 NumLogicalProcessors = FPlatformMisc::NumberOfCoresIncludingHyperthreads();
	const int32 NumCores = FPlatformMisc::NumberOfCores();
	bProcessorSMT = NumLogicalProcessors > NumCores;

	const FIPv4Address DefaultIp = FIPv4Address(0, 0, 0, 0);
	if (!Options.Address.IsSet())
	{
		UE_LOGF(LogSwitchboard, Warning, "Defaulting to: -ip=%ls", *DefaultIp.ToString());
	}

	const uint16 DefaultPort = 2980;
	if (!Options.Port.IsSet())
	{
		UE_LOGF(LogSwitchboard, Warning, "Defaulting to: -port=%u", DefaultPort);
	}

	ListenerEndpoint = MakeUnique<FIPv4Endpoint>(Options.Address.Get(DefaultIp), Options.Port.Get(DefaultPort));
}

FSwitchboardListener::~FSwitchboardListener()
{
}

bool FSwitchboardListener::Init()
{
	if (!AuthHelper.Initialize())
	{
		return false;
	}

	OnInitDelegate.Broadcast();

	return true;
}


void FSwitchboardListener::Shutdown()
{
	AuthHelper.Shutdown();

	OnShutdownDelegate.Broadcast();
}


const UTF8CHAR* FSwitchboardListener::GetAuthPassword() const
{
	return AuthHelper.GetAuthPassword();
}


bool FSwitchboardListener::IsAuthPasswordSet() const
{
	return AuthHelper.IsAuthPasswordSet();
}


bool FSwitchboardListener::SetAuthPassword(const FString& NewPassword)
{
	return AuthHelper.SetAuthPassword(NewPassword);
}


TSet<FIPv4Endpoint> FSwitchboardListener::GetConnectedClientEndpoints() const
{
	TSet<FIPv4Endpoint> OutEndpointSet;
	ConnectionsByEndpoint.GetKeys(OutEndpointSet);
	return OutEndpointSet;
}


TSet<FIPv4Address> FSwitchboardListener::GetConnectedClientAddresses() const
{
	TSet<FIPv4Address> OutAddressSet;
	for (const TPair<FIPv4Endpoint, FConnectionRef>& Endpoint : ConnectionsByEndpoint)
	{
		OutAddressSet.Add(Endpoint.Key.Address);
	}
	return OutAddressSet;
}


bool FSwitchboardListener::StartListening()
{
	QUIC_STATUS Status;
	if (QUIC_FAILED(Status = MsQuicOpen2(&QuicApi)))
	{
		UE_LOGFMT(LogSwitchboard, Error, "MsQuicOpen2 failed with status: {Status}", *GetQuicErrorStr(Status));
		return false;
	}

	// Create a registration for the app's connections. This sets a name for the
	// app (used for persistent storage and for debugging). It also configures
	// the execution profile, using the default "low latency" profile.
	const QUIC_REGISTRATION_CONFIG RegConfig = { "switchboardlistener", QUIC_EXECUTION_PROFILE_LOW_LATENCY };
	if (QUIC_FAILED(Status = QuicApi->RegistrationOpen(&RegConfig, &QuicRegistration))) {
		UE_LOGFMT(LogSwitchboard, Error, "MsQuic RegistrationOpen failed with status: {Status}", *GetQuicErrorStr(Status));
		return false;
	}

	QUIC_CERTIFICATE_FILE CertFile = { 0 };
	QUIC_CERTIFICATE_FILE_PROTECTED CertFileProtected = { 0 };
	QUIC_CREDENTIAL_CONFIG CredConfig = {
		.Type = QUIC_CREDENTIAL_TYPE_NONE,
		.Flags =
			QUIC_CREDENTIAL_FLAG_INDICATE_CERTIFICATE_RECEIVED |
			QUIC_CREDENTIAL_FLAG_USE_PORTABLE_CERTIFICATES,
	};

	if (AuthHelper.GetPrivateKeyPassword())
	{
		CertFileProtected.PrivateKeyFile = reinterpret_cast<const char*>(*AuthHelper.GetPrivateKeyFilePath());
		CertFileProtected.CertificateFile = reinterpret_cast<const char*>(*AuthHelper.GetCertificateFilePath());
		CertFileProtected.PrivateKeyPassword = reinterpret_cast<const char*>(AuthHelper.GetPrivateKeyPassword());

		CredConfig.Type = QUIC_CREDENTIAL_TYPE_CERTIFICATE_FILE_PROTECTED;
		CredConfig.CertificateFileProtected = &CertFileProtected;
	}
	else
	{
		CertFile.PrivateKeyFile = reinterpret_cast<const char*>(*AuthHelper.GetPrivateKeyFilePath());
		CertFile.CertificateFile = reinterpret_cast<const char*>(*AuthHelper.GetCertificateFilePath());

		CredConfig.Type = QUIC_CREDENTIAL_TYPE_CERTIFICATE_FILE;
		CredConfig.CertificateFile = &CertFile;
	}

	QUIC_SETTINGS Settings = {};

	Settings.IdleTimeoutMs = DefaultInactiveTimeoutSeconds * 1000.0;
	Settings.IsSet.IdleTimeoutMs = 1;

	// Allow each peer to open a single bidirectional stream.
	Settings.PeerBidiStreamCount = 1;
	Settings.IsSet.PeerBidiStreamCount = 1;

	// The protocol name used in Application Layer Protocol Negotiation.
	constexpr char SblAlpnStr[] = "ue-switchboard";
	const QUIC_BUFFER SblAlpn = { sizeof(SblAlpnStr) - 1, (uint8_t*)SblAlpnStr };

	// Allocate/initialize the configuration object with the configured ALPN and settings.
	if (QUIC_FAILED(Status = QuicApi->ConfigurationOpen(QuicRegistration, &SblAlpn, 1, &Settings, sizeof(Settings), NULL, &QuicConfiguration)))
	{
		UE_LOGFMT(LogSwitchboard, Error, "MsQuic ConfigurationOpen failed with status: {Status}", *GetQuicErrorStr(Status));
		return false;
	}

	// Loads the TLS credential part of the configuration.
	if (QUIC_FAILED(Status = QuicApi->ConfigurationLoadCredential(QuicConfiguration, &CredConfig)))
	{
		UE_LOGFMT(LogSwitchboard, Error, "MsQuic ConfigurationLoadCredential failed with status: {Status}", *GetQuicErrorStr(Status));
		return false;
	}

	// Create/allocate a new listener object.
	if (QUIC_FAILED(Status = QuicApi->ListenerOpen(QuicRegistration, QuicListenerThunk, this, &QuicListener))) 
	{
		UE_LOGFMT(LogSwitchboard, Error, "MsQuic ListenerOpen failed with status: {Status}", *GetQuicErrorStr(Status));
		return false;
	}

	// Starts listening for incoming connections.
	QUIC_ADDR QuicAddr = QuicAddrFromEndpoint(*ListenerEndpoint);
	if (QUIC_FAILED(Status = QuicApi->ListenerStart(QuicListener, &SblAlpn, 1, &QuicAddr))) 
	{
		UE_LOGFMT(LogSwitchboard, Error, "MsQuic ListenerStart failed with status: {Status}", *GetQuicErrorStr(Status));
		return false;
	}

#if PLATFORM_WINDOWS
	// If we've made it this far, we were able to bind the listen endpoint successfully,
	// which precludes interfering with any other running instance.
	HarvestExistingFlipModeSessions();
#endif

	return true;
}

bool FSwitchboardListener::StopListening()
{
	if (QuicApi)
	{
		if (QuicListener)
		{
			QuicApi->ListenerClose(QuicListener);
			QuicListener = nullptr;
		}

		if (QuicConfiguration)
		{
			QuicApi->ConfigurationClose(QuicConfiguration);
			QuicConfiguration = nullptr;
		}

		if (QuicRegistration)
		{
			// This will block until all outstanding child objects have been closed.
			QuicApi->RegistrationClose(QuicRegistration);
			QuicRegistration = nullptr;
		}

		MsQuicClose(QuicApi);
		QuicApi = nullptr;
	}

	return true;
}

void FSwitchboardListener::Tick()
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::Tick);

	// Parse incoming data from remote connections
	{
		UE::TUniqueLock<UE::FRecursiveMutex> ConnectionsScopeLock(ConnectionsLock);
		for (const TPair<FIPv4Endpoint, FConnectionRef>& ConnectionPair : ConnectionsByEndpoint)
		{
			const FIPv4Endpoint& ClientEndpoint = ConnectionPair.Key;
			FConnectionRef Connection = ConnectionPair.Value;
			if (Connection->bMessageComplete && Connection->ReceiveLock.TryLock())
			{
				ON_SCOPE_EXIT{ Connection->ReceiveLock.Unlock(); };

				const FString Message(UTF8_TO_TCHAR(Connection->ReceiveBuffer->GetData()));
				ParseIncomingMessage(Message, ClientEndpoint, Connection);
				Connection->ReceiveBuffer->Empty();
				Connection->bMessageComplete = false;

				// Resume receive events after processing the pending message.
				QuicApi->StreamReceiveSetEnabled(Connection->QuicStream, true);
			}
		}
	}

	// Run the next queued task
	if (!ScheduledTasks.IsEmpty())
	{
		TUniquePtr<FSwitchboardTask> Task;
		ScheduledTasks.Dequeue(Task);
		RunScheduledTask(*Task);
	}

	CleanUpDisconnectedSockets();
	HandleRunningProcesses(RunningProcesses, true);
	HandleRunningProcesses(FlipModeMonitors, false);
	HandleLiveLogStreams();
	SendMessageFutures();
	GpuClockerManager->Tick();

#if PLATFORM_WINDOWS
	FWorkaroundForHitchingAfterHours::Get().Tick();
#endif // PLATFORM_WINDOWS

	OnTickDelegate.Broadcast();
}

bool FSwitchboardListener::ParseIncomingMessage(const FString& InMessage, const FIPv4Endpoint& InEndpoint, const FConnectionRef& Connection)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::ParseIncomingMessage);

	FCreateTaskResult CreateTaskResult =
		CreateTaskFromCommand(InMessage, InEndpoint, Connection->bAuthenticated);

	if (CreateTaskResult.Status == ECreateTaskStatus::Success)
	{
		if (CreateTaskResult.Task->Type == ESwitchboardTaskType::Disconnect)
		{
			DisconnectTasks.Enqueue(MoveTemp(CreateTaskResult.Task));
		}
		else if (CreateTaskResult.Task->Type == ESwitchboardTaskType::KeepAlive)
		{
			UE::TUniqueLock<UE::FRecursiveMutex> ConnectionsScopeLock(ConnectionsLock);
			ConnectionsByEndpoint[InEndpoint]->LastActivityTime = FPlatformTime::Seconds();
		}
		else
		{
			if (CreateTaskResult.bEcho)
			{
				UE_LOGF(LogSwitchboard, Display, "Received %ls command", CreateTaskResult.Task->GetCommandName());
			}

			SendMessage(CreateCommandAcceptedMessage(CreateTaskResult.Task->TaskID), InEndpoint);
			ScheduledTasks.Enqueue(MoveTemp(CreateTaskResult.Task));
		}
		return true;
	}
	else if (CreateTaskResult.Status == ECreateTaskStatus::Error_Unauthenticated)
	{
		static FString UnauthenticatedError = FString::Printf(TEXT("Rejecting '%s' command for unauthenticated client %s"),
			*CreateTaskResult.CommandName.Get(TEXT("(?)")), *InEndpoint.ToString());

		UE_LOGF(LogSwitchboard, Error, "%ls", *UnauthenticatedError);

		// just use an empty ID if we couldn't find one
		SendMessage(CreateCommandDeclinedMessage(CreateTaskResult.MessageID.Get(FGuid()), UnauthenticatedError), InEndpoint);
	}
	else
	{
		if (!CreateTaskResult.MessageID.IsSet())
		{
			FGuid RecoveredID;
			if (TryFindIdInBrokenMessage(InMessage, RecoveredID))
			{
				CreateTaskResult.MessageID = RecoveredID;
			}
		}

		static FString ParseError = FString::Printf(TEXT("Could not parse message %s with ID %s"),
			*InMessage,
			CreateTaskResult.MessageID.IsSet() ? *CreateTaskResult.MessageID.GetValue().ToString() : TEXT("(?)"));
		UE_LOGF(LogSwitchboard, Error, "%ls", *ParseError);
			
		// just use an empty ID if we couldn't find one
		SendMessage(CreateCommandDeclinedMessage(CreateTaskResult.MessageID.Get(FGuid()), ParseError), InEndpoint);
	}

	return false;
}

bool FSwitchboardListener::RunScheduledTask(const FSwitchboardTask& InTask)
{
	switch (InTask.Type)
	{
		case ESwitchboardTaskType::Authenticate:
		{
			const FSwitchboardAuthenticateTask& AuthTask = static_cast<const FSwitchboardAuthenticateTask&>(InTask);
			return Task_Authenticate(AuthTask);
		}
		case ESwitchboardTaskType::Start:
		{
			const FSwitchboardStartTask& StartTask = static_cast<const FSwitchboardStartTask&>(InTask);
			return Task_StartProcess(StartTask);
		}
		case ESwitchboardTaskType::Kill:
		{
			const FSwitchboardKillTask& KillTask = static_cast<const FSwitchboardKillTask&>(InTask);
			return Task_KillProcess(KillTask);
		}
		case ESwitchboardTaskType::ReceiveFileFromClient:
		{
			const FSwitchboardReceiveFileFromClientTask& ReceiveFileFromClientTask = static_cast<const FSwitchboardReceiveFileFromClientTask&>(InTask);
			return Task_ReceiveFileFromClient(ReceiveFileFromClientTask);
		}
		case ESwitchboardTaskType::RedeployListener:
		{
			const FSwitchboardRedeployListenerTask& RedeployListenerTask = static_cast<const FSwitchboardRedeployListenerTask&>(InTask);
			const bool bRedeployOk = Task_RedeployListener(RedeployListenerTask);
			if (!bRedeployOk)
			{
				RollbackRedeploy();
			}
			return bRedeployOk;
		}
		case ESwitchboardTaskType::SendFileToClient:
		{
			const FSwitchboardSendFileToClientTask& SendFileToClientTask = static_cast<const FSwitchboardSendFileToClientTask&>(InTask);
			return Task_SendFileToClient(SendFileToClientTask);
		}
		case ESwitchboardTaskType::KeepAlive:
		{
			return true;
		}
		case ESwitchboardTaskType::GetSyncStatus:
		{
			const FSwitchboardGetSyncStatusTask& GetSyncStatusTask = static_cast<const FSwitchboardGetSyncStatusTask&>(InTask);
			return Task_GetSyncStatus(GetSyncStatusTask);
		}
		case ESwitchboardTaskType::FixExeFlags:
		{
			const FSwitchboardFixExeFlagsTask& Task = static_cast<const FSwitchboardFixExeFlagsTask&>(InTask);
			return Task_FixExeFlags(Task);
		}
		case ESwitchboardTaskType::RefreshMosaics:
		{
			const FSwitchboardRefreshMosaicsTask& Task = static_cast<const FSwitchboardRefreshMosaicsTask&>(InTask);
			return Task_RefreshMosaics(Task);
		}
		case ESwitchboardTaskType::MinimizeWindows:
		{
			const FSwitchboardMinimizeWindowsTask& Task = static_cast<const FSwitchboardMinimizeWindowsTask&>(InTask);
			return Task_MinimizeWindows(Task);
		}
		case ESwitchboardTaskType::SetInactiveTimeout:
		{
			const FSwitchboardSetInactiveTimeoutTask& Task = static_cast<const FSwitchboardSetInactiveTimeoutTask&>(InTask);
			return Task_SetInactiveTimeout(Task);
		}
		case ESwitchboardTaskType::FreeListenerBinary:
		{
			const FSwitchboardFreeListenerBinaryTask& Task = static_cast<const FSwitchboardFreeListenerBinaryTask&>(InTask);
			return Task_FreeListenerBinary(Task);
		}
		case ESwitchboardTaskType::StartLiveLogStream:
		{
			const FSwitchboardStartLiveLogStreamTask& Task = static_cast<const FSwitchboardStartLiveLogStreamTask&>(InTask);
			return Task_StartLiveLogStream(Task);
		}
		case ESwitchboardTaskType::StopLiveLogStream:
		{
			const FSwitchboardStopLiveLogStreamTask& Task = static_cast<const FSwitchboardStopLiveLogStreamTask&>(InTask);
			return Task_StopLiveLogStream(Task);
		}
		default:
		{
			static const FString Response = TEXT("Unknown Command detected");
			CreateCommandDeclinedMessage(InTask.TaskID, Response);
			return false;
		}
	}
}

static uint32 FindPidInFocus()
{
#if PLATFORM_WINDOWS
	HWND WindowHandle = GetForegroundWindow();

	if (WindowHandle)
	{
		DWORD PID = 0;
		DWORD ThreadId = GetWindowThreadProcessId(WindowHandle, &PID);

		if (ThreadId != 0)
		{
			return PID;
		}
	}
#endif // PLATFORM_WINDOWS

	return 0;
}

#if PLATFORM_WINDOWS
static TArray<FString> RegistryGetValueNames(const HKEY Key)
{
	TArray<FString> Names;
	const uint32 MaxLength = 1024;
	TCHAR ValueName[MaxLength];
	DWORD ValueLength = MaxLength;

	while (!RegEnumValue(Key, Names.Num(), ValueName, &ValueLength, nullptr, nullptr, nullptr, nullptr))
	{
		Names.Add(ValueName);
		ValueLength = MaxLength;
	}

	return Names;
}
#endif // PLATFORM_WINDOWS

#if PLATFORM_WINDOWS
static FString RegistryGetStringValueData(const HKEY Key, const FString& ValueName)
{
	const uint32 MaxLength = 4096;
	TCHAR ValueData[MaxLength];
	DWORD ValueLength = MaxLength;

	if (RegQueryValueEx(Key, *ValueName, 0, 0, LPBYTE(ValueData), &ValueLength))
	{
		return TEXT("");
	}

	ValueData[MaxLength - 1] = '\0';

	return FString(ValueData);
}
#endif // PLATFORM_WINDOWS

#if PLATFORM_WINDOWS
static bool RegistrySetStringValueData(const HKEY Key, const FString& ValueName, const FString& ValueData)
{
	return !!RegSetValueEx(Key, *ValueName, 0, REG_SZ, LPBYTE(*ValueData), sizeof(TCHAR) * ValueData.Len());
}
#endif // PLATFORM_WINDOWS

static bool DisableFullscreenOptimizationForProcess(const FRunningProcess* Process)
{
#if !PLATFORM_WINDOWS
	return false;
#else
	// No point in continuing if there is no process to set the flags for.
	if (!Process)
	{
		return false;
	}

	// This is the absolute path of the program we'll be looking for in the registry
	const FString ProcessAbsolutePath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*Process->Path);
	const FString DisableFsOptLayer = TEXT("DISABLEDXMAXIMIZEDWINDOWEDMODE");
	const FString LayersKeyPath = TEXT("Software\\Microsoft\\Windows NT\\CurrentVersion\\AppCompatFlags\\Layers");

	// Check if the key exists
	HKEY LayersKey;
	if (RegOpenKeyExW(HKEY_CURRENT_USER, *LayersKeyPath, 0, KEY_ALL_ACCESS, &LayersKey))
	{
		return false;
	}

	ON_SCOPE_EXIT
	{
		RegCloseKey(LayersKey);
	};

	// If the key exists, the Value Names are the paths to the programs
	const TArray<FString> ProgramPaths = RegistryGetValueNames(LayersKey);
	const FString* ExistingPath = Algo::FindBy(ProgramPaths, ProcessAbsolutePath, [](const FString& ProgPath) { return IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ProgPath); });
	if (ExistingPath)
	{
		// If the program already has an entry, get the layers from the Value Data.
		FString ProgramLayers = RegistryGetStringValueData(LayersKey, *ExistingPath);

		TArray<FString> ProgramLayersArray;
		ProgramLayers.ParseIntoArray(ProgramLayersArray, TEXT(" "));

		if (ProgramLayersArray.Contains(DisableFsOptLayer))
		{
			// Layer already present, nothing to do.
			return true;
		}

		// Append the desired layer.
		ProgramLayers = FString::Printf(TEXT("%s %s"), *ProgramLayers, *DisableFsOptLayer);
		return RegistrySetStringValueData(LayersKey, *ExistingPath, ProgramLayers);
	}
	else
	{
		// The path to our executable does not exist as one of the regkey values, so we need to create it.
		const FString ProgramLayers = FString::Printf(TEXT("~ %s"), *DisableFsOptLayer);
		const FString ProgramAbsPath = ProcessAbsolutePath.Replace(TEXT("/"), TEXT("\\"));
		return RegistrySetStringValueData(LayersKey, ProgramAbsPath, ProgramLayers);
	}
#endif // PLATFORM_WINDOWS
}

bool FSwitchboardListener::Task_Authenticate(const FSwitchboardAuthenticateTask& InAuthTask)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::Task_Authenticate);

	UE::TUniqueLock<UE::FRecursiveMutex> ConnectionsScopeLock(ConnectionsLock);
	FConnectionRef Connection = ConnectionsByEndpoint[InAuthTask.Recipient];

	// If we decide later to issue a new JWT, we include it in SendAuthResponse.
	TOptional<FString> IssueJwt;

	auto SendAuthResponse = [this, &InAuthTask, &Connection, &IssueJwt]()
		{
			TMap<FString, FString> AuthResponse =
			{
				{ TEXT("command"), FSwitchboardAuthenticateTask::CommandName },
				{ TEXT("id"), InAuthTask.TaskID.ToString() },
				{ TEXT("bAuthenticated"), Connection->bAuthenticated ? TEXT("true") : TEXT("false") },
			};

			if (IssueJwt)
			{
				AuthResponse.Add(TEXT("jwt"), *IssueJwt);
			}

			SendMessage(CreateMessage(AuthResponse), InAuthTask.Recipient);

			if (Connection->bAuthenticated)
			{
				// Send current state upon authentication
				{
					FSwitchboardStatePacket StatePacket;
					FillStatePacket(StatePacket);
					SendMessage(CreateMessage(StatePacket), InAuthTask.Recipient);
				}
			}
		};

	if (Connection->bAuthenticated)
	{
		UE_LOGFMT(LogSwitchboard, Warning, "Ignoring redundant auth attempt for authenticated client {Endpoint}", Connection->Endpoint.ToString());
		SendAuthResponse();
		return false;
	}

	int8& AuthFailures = AuthFailuresByAddress.FindOrAdd(InAuthTask.Recipient.Address);
	if (AuthFailures >= MaxAuthFailures)
	{
		UE_LOGFMT(LogSwitchboard, Error, "Ignoring auth attempt for client with excessive failures {Endpoint}", Connection->Endpoint.ToString());
		SendAuthResponse();
		return false;
	}

	if (InAuthTask.Jwt)
	{
		if (AuthHelper.IsValidJWT(*InAuthTask.Jwt))
		{
			UE_LOGFMT(LogSwitchboard, Display, "Client {Endpoint} authenticated successfully (using token)", Connection->Endpoint.ToString());
			Connection->bAuthenticated = true;
			SendAuthResponse();
			return true;
		}
	}
	else if (InAuthTask.Password)
	{
		if (AuthHelper.ValidatePassword(*InAuthTask.Password))
		{
			UE_LOGFMT(LogSwitchboard, Display, "Client {Endpoint} authenticated successfully (using password)", Connection->Endpoint.ToString());
			Connection->bAuthenticated = true;
			IssueJwt = AuthHelper.IssueJWT({});
			SendAuthResponse();
			return true;
		}
	}

	UE_LOGFMT(LogSwitchboard, Warning, "Failed authentication attempt for client {Endpoint}", Connection->Endpoint.ToString());
	++AuthFailures;
	SendAuthResponse();
	return false;
}

bool FSwitchboardListener::Task_StartProcess(const FSwitchboardStartTask& InRunTask)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::Task_StartProcess);

	TSharedRef<FRunningProcess> NewProcess = MakeShared<FRunningProcess>();

	NewProcess->Recipient = InRunTask.Recipient;
	NewProcess->Path = InRunTask.Command;
	NewProcess->Name = InRunTask.Name;
	NewProcess->Caller = InRunTask.Caller;
	NewProcess->bUpdateClientsWithStdout = InRunTask.bUpdateClientsWithStdout;
	NewProcess->bCombinedStdoutStderr = InRunTask.bCombinedStdoutStderr;
	NewProcess->bLockGpuClock = InRunTask.bLockGpuClock;
	NewProcess->UUID = InRunTask.TaskID; // Process ID is the same as the message ID.
	NewProcess->PID = 0; // default value

	if (!NewProcess->CreatePipes())
	{
		return false;
	}

	const bool bLaunchDetached = false;
	const bool bLaunchHidden = InRunTask.bHide;
	const bool bLaunchReallyHidden = InRunTask.bHide;
	const int32 PriorityModifier = InRunTask.PriorityModifier;
	const TCHAR* WorkingDirectory = InRunTask.WorkingDir.IsEmpty() ? nullptr : *InRunTask.WorkingDir;

	NewProcess->Handle = FSwitchboardProcess::CreateProc(
		*InRunTask.Command, 
		*InRunTask.Arguments, 
		bLaunchDetached, 
		bLaunchHidden, 
		bLaunchReallyHidden, 
		&NewProcess->PID, 
		PriorityModifier, 
		WorkingDirectory, 
		NewProcess->StdoutChildWritePipe,
		NewProcess->StdinChildReadPipe,
		InRunTask.bCombinedStdoutStderr ? NewProcess->StdoutChildWritePipe : NewProcess->StderrChildWritePipe
	);

	if (!NewProcess->Handle.IsValid())
	{
		// Close process in case it just didn't run
		FPlatformProcess::CloseProc(NewProcess->Handle);

		// close pipes
		NewProcess->ClosePipes();

		// log error
		const FString ErrorMsg = FString::Printf(TEXT("Could not start program %s"), *InRunTask.Command);
		UE_LOGF(LogSwitchboard, Error, "%ls", *ErrorMsg);

		// notify Switchboard
		SendMessage(
			CreateTaskDeclinedMessage(
				InRunTask, 
				ErrorMsg, 
				{
					{ TEXT("puuid"), NewProcess->UUID.ToString() },
				}
			), 
			InRunTask.Recipient
		);

		return false;
	}

	// Lock Gpu Clocks for the lifetime of this PID, if requested
	if (InRunTask.bLockGpuClock && GpuClockerManager.IsValid())
	{
		GpuClockerManager->LockGpuClocksForPid(NewProcess->PID);
	}

	UE_LOGF(LogSwitchboard, Display, "Started process %d: %ls %ls", NewProcess->PID, *InRunTask.Command, *InRunTask.Arguments);

	RunningProcesses.Add(NewProcess);

	// send message
	{
		FSwitchboardProgramStarted Packet;

		Packet.Process.Uuid = NewProcess->UUID.ToString();
		Packet.Process.Name = NewProcess->Name;
		Packet.Process.Path = NewProcess->Path;
		Packet.Process.Caller = NewProcess->Caller;
		Packet.Process.Pid = NewProcess->PID;

		UE::TUniqueLock<UE::FRecursiveMutex> ConnectionsScopeLock(ConnectionsLock);
		for (const TPair<FIPv4Endpoint, FConnectionRef>& Connection : ConnectionsByEndpoint)
		{
			const FIPv4Endpoint& ClientEndpoint = Connection.Key;
			SendMessage(CreateMessage(Packet), ClientEndpoint);
		}
	}

	return true;
}

bool FSwitchboardListener::Task_KillProcess(const FSwitchboardKillTask& KillTask)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::Task_KillProcess);

	if (EquivalentTaskFutureExists(KillTask.GetEquivalenceHash()))
	{
		SendMessage(
			CreateTaskDeclinedMessage(
				KillTask,
				TEXT("Duplicate"),
				{
					{ TEXT("puuid"), KillTask.ProgramID.ToString() },
				}
			),
			KillTask.Recipient
		);

		return false;
	}

	// Look in RunningProcesses
	TSharedPtr<FRunningProcess> Process;
	{
		TSharedPtr<FRunningProcess>* ProcessPtr = RunningProcesses.FindByPredicate([&KillTask](const TSharedPtr<FRunningProcess>& InProcess)
		{
			check(InProcess.IsValid());
			return !InProcess->bPendingKill && (InProcess->UUID == KillTask.ProgramID);
		});

		if (ProcessPtr)
		{
			Process = *ProcessPtr;
			Process->bPendingKill = true;
		}
	}

	// Look in FlipModeMonitors
	TSharedPtr<FRunningProcess> FlipModeMonitor;
	{
		TSharedPtr<FRunningProcess>* FlipModeMonitorPtr = FlipModeMonitors.FindByPredicate([&](const TSharedPtr<FRunningProcess>& FlipMonitor)
		{
			check(FlipMonitor.IsValid());
			return !FlipMonitor->bPendingKill && (FlipMonitor->UUID == KillTask.ProgramID);
		});

		if (FlipModeMonitorPtr)
		{
			FlipModeMonitor = *FlipModeMonitorPtr;
			FlipModeMonitor->bPendingKill = true;
		}
	}

	// Create our future message
	FSwitchboardMessageFuture MessageFuture;

	MessageFuture.InEndpoint = KillTask.Recipient;
	MessageFuture.EquivalenceHash = KillTask.GetEquivalenceHash();

	const FGuid UUID = KillTask.ProgramID;

	MessageFuture.Future = Async(EAsyncExecution::ThreadPool, [this, KillTask, Process, FlipModeMonitor, UUID]() {
		SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE_STR("FSwitchboardListener::Task_KillProcess future closure");

		const float SoftKillTimeout = FMath::Max(0.0f, KillTask.SoftKillTimeout);

		const bool bKilledProcess = KillProcessNow(Process.Get(), SoftKillTimeout);
		KillProcessNow(FlipModeMonitor.Get(), SoftKillTimeout);

		// Clear bPendingKill
		if (Process)
		{
			Process->bPendingKill = false;
		}

		if (FlipModeMonitor)
		{
			FlipModeMonitor->bPendingKill = false;
		}

		// Return message
		const FString ProgramID = UUID.ToString();

		if (!bKilledProcess)
		{
			const FString KillError = FString::Printf(TEXT("Could not kill program with ID %s"), *ProgramID);

			return CreateTaskDeclinedMessage(
				KillTask,
				KillError,
				{
					{ TEXT("puuid"), ProgramID },
				}
			);
		}

		FSwitchboardProgramKilled Packet;

		if (Process) // must be valid if it was killed
		{
			Packet.Process.Uuid = Process->UUID.ToString();
			Packet.Process.Name = Process->Name;
			Packet.Process.Path = Process->Path;
			Packet.Process.Caller = Process->Caller;
			Packet.Process.Pid = Process->PID;
		}

		return CreateMessage(Packet);
	});

	// Queue it to be sent when ready
	MessagesFutures.Emplace(MoveTemp(MessageFuture));

	return true;
}

bool FSwitchboardListener::Task_FixExeFlags(const FSwitchboardFixExeFlagsTask& Task)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::Task_FixExeFlags);

	if (EquivalentTaskFutureExists(Task.GetEquivalenceHash()))
	{
		SendMessage(	
			CreateTaskDeclinedMessage(
				Task,
				TEXT("Duplicate"),
				{
					{ TEXT("puuid"), Task.ProgramID.ToString() },
				}
			),
			Task.Recipient
		);

		return false;
	}

	// Look in RunningProcesses
	TSharedPtr<FRunningProcess> Process;
	{
		TSharedPtr<FRunningProcess>* ProcessPtr = RunningProcesses.FindByPredicate([&Task](const TSharedPtr<FRunningProcess>& InProcess)
			{
				check(InProcess.IsValid());
				return !InProcess->bPendingKill && (InProcess->UUID == Task.ProgramID);
			});

		if (ProcessPtr)
		{
			Process = *ProcessPtr;
		}
	}

	if (!Process.IsValid())
	{
		SendMessage(
			CreateTaskDeclinedMessage(
				Task,
				TEXT("Could not find ProgramID"),
				{
					{ TEXT("puuid"), Task.ProgramID.ToString() },
				}
			),
			Task.Recipient
		);

		return false;
	}

	return DisableFullscreenOptimizationForProcess(Process.Get());
}


#if PLATFORM_WINDOWS

/**
 * Recursively collects the PIDs of all child processes of a given root PID.
 */
static void WindowsGatherChildPIDs(uint32 RootPID, TArray<uint32>& OutPIDs)
{
	const HANDLE SnapshotHandle = CreateToolhelp32Snapshot(TH32CS_SNAPPROCESS, 0);

	if (SnapshotHandle == INVALID_HANDLE_VALUE)
	{
		return;
	}

	PROCESSENTRY32 ProcessEntry;
	ProcessEntry.dwSize = sizeof(ProcessEntry);

	if (Process32First(SnapshotHandle, &ProcessEntry))
	{
		do
		{
			if (ProcessEntry.th32ParentProcessID == RootPID)
			{
				OutPIDs.Add(ProcessEntry.th32ProcessID);
				WindowsGatherChildPIDs(ProcessEntry.th32ProcessID, OutPIDs); // Recursion
			}
		} while (Process32Next(SnapshotHandle, &ProcessEntry));
	}

	CloseHandle(SnapshotHandle);
}


/** 
 * Soft-terminates a process by PID, excluding its process tree. 
 */
static bool WindowsSoftTerminateByPID(uint32 PID)
{
	const FString Params = FString::Printf(TEXT("/PID %u"), PID);

	FString StdOut;
	int32 ReturnCode = 0;

	FPlatformProcess::ExecProcess(TEXT("TASKKILL"), *Params, &ReturnCode, &StdOut, nullptr);

	return (ReturnCode == 0);
}


/**
 * Terminate a process tree by first sending individual soft sins to each PID,
 * waiting for up to SoftKillTimeout seconds, then forcefully terminating survivors.
 */
bool WindowsKillProcessNow(FRunningProcess* InProcess, float SoftKillTimeout)
{
	check(InProcess && InProcess->Handle.IsValid()); // We expect this was checked before calling this platform specific function.

	// Gather all PIDs to terminate, including the root.

	TArray<uint32> AllPIDs;
	AllPIDs.Add(InProcess->PID);

	WindowsGatherChildPIDs(InProcess->PID, AllPIDs);

	// Soft terminate each one

	for (uint32 PID : AllPIDs)
	{
		WindowsSoftTerminateByPID(PID);
	}

	// Wait for them to terminate, up to the given timeout

	const double DeadlineTime = FPlatformTime::Seconds() + SoftKillTimeout;

	while (FPlatformTime::Seconds() < DeadlineTime)
	{
		bool bAnyStillRunning = false;

		for (uint32 PID : AllPIDs)
		{
			const HANDLE ProcessHandle = ::OpenProcess(SYNCHRONIZE, false, PID);

			if (ProcessHandle)
			{
				DWORD ExitCode = 0;

				if (::GetExitCodeProcess(ProcessHandle, &ExitCode) && ExitCode == STILL_ACTIVE)
				{
					bAnyStillRunning = true;
				}

				::CloseHandle(ProcessHandle);
			}

			if (bAnyStillRunning)
			{
				break;
			}
		}

		if (!bAnyStillRunning)
		{
			break;
		}

		FPlatformProcess::Sleep(0.05f);
	}

	// Hard‐terminate surviving PIDs

	for (uint32 PID : AllPIDs)
	{
		FProcHandle TermHandle = FPlatformProcess::OpenProcess(PID);

		if (TermHandle.IsValid())
		{
			if (FPlatformProcess::IsProcRunning(TermHandle))
			{
				FPlatformProcess::TerminateProc(TermHandle, false /* bKillTree */);
			}
			FPlatformProcess::CloseProc(TermHandle);
		}
	}

	return true;
}
#endif // PLATFORM_WINDOWS


bool FSwitchboardListener::KillProcessNow(FRunningProcess* InProcess, float SoftKillTimeout)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::KillProcessNow);

	if (!InProcess || !InProcess->Handle.IsValid() || !FPlatformProcess::IsProcRunning(InProcess->Handle))
	{
		return false;
	}

	UE_LOGF(LogSwitchboard, Display, "Killing app with PID %d", InProcess->PID);

#if PLATFORM_WINDOWS

	return WindowsKillProcessNow(InProcess, SoftKillTimeout);

#else

	if (FPlatformProcess::IsProcRunning(InProcess->Handle))
	{
		FPlatformProcess::TerminateProc(InProcess->Handle, true /* bKillTree */);
	}

	// Pipes will be closed in HandleRunningProcesses
	return true;

#endif // PLATFORM_WINDOWS
}

bool FSwitchboardListener::Task_ReceiveFileFromClient(const FSwitchboardReceiveFileFromClientTask& InReceiveFileFromClientTask)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::Task_ReceiveFileFromClient);

	FString Destination = InReceiveFileFromClientTask.Destination;

	if (Destination.Contains(TEXT("%TEMP%")))
	{
		FString TempDir = FPlatformProcess::UserTempDir();

		if (TempDir.EndsWith(TEXT("/")) || TempDir.EndsWith(TEXT("\\")))
		{
			TempDir.LeftChopInline(1, EAllowShrinking::No);
		}

		Destination.ReplaceInline(TEXT("%TEMP%"), *TempDir);
	}
	if (Destination.Contains(TEXT("%RANDOM%")))
	{
		const FString Path = FPaths::GetPath(Destination);
		const FString Extension = FPaths::GetExtension(Destination, true);
		Destination = FPaths::CreateTempFilename(*Path, TEXT(""), *Extension);
	}
	FPlatformMisc::NormalizePath(Destination);
	FPaths::MakePlatformFilename(Destination);

	if (FPaths::FileExists(Destination))
	{
		if (!InReceiveFileFromClientTask.bForceOverwrite)
		{
			const FString ErrorMsg = FString::Printf(TEXT("Destination %s already exist"), *Destination);
			UE_LOGF(LogSwitchboard, Error, "%ls", *ErrorMsg);
			SendMessage(CreateReceiveFileFromClientFailedMessage(Destination, ErrorMsg), InReceiveFileFromClientTask.Recipient);
			return false;
		}
		else
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			PlatformFile.SetReadOnly(*Destination, false);
		}
	}

	TArray<uint8> DecodedFileContent = {};
	FBase64::Decode(InReceiveFileFromClientTask.FileContent, DecodedFileContent);

	UE_LOGF(LogSwitchboard, Display, "Writing %d bytes to %ls", DecodedFileContent.Num(), *Destination);
	if (FFileHelper::SaveArrayToFile(DecodedFileContent, *Destination))
	{
		SendMessage(CreateReceiveFileFromClientCompletedMessage(Destination), InReceiveFileFromClientTask.Recipient);
		return true;
	}

	const FString ErrorMsg = FString::Printf(TEXT("Error while trying to write to %s"), *Destination);
	UE_LOGF(LogSwitchboard, Error, "%ls", *ErrorMsg);
	SendMessage(CreateReceiveFileFromClientFailedMessage(Destination, ErrorMsg), InReceiveFileFromClientTask.Recipient);
	return false;
}

bool FSwitchboardListener::Task_RedeployListener(const FSwitchboardRedeployListenerTask& InRedeployListenerTask)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::Task_RedeployListener);

	if (RedeployStatus.State != FRedeployStatus::EState::None)
	{
		const FString ErrorMsg(TEXT("Redeploy already in progress"));
		UE_LOGF(LogSwitchboard, Error, "Redeploy listener: %ls", *ErrorMsg);
		SendMessage(CreateRedeployStatusMessage(InRedeployListenerTask.TaskID, false, ErrorMsg), InRedeployListenerTask.Recipient);
		return false;
	}

	RedeployStatus.RequestingClient = InRedeployListenerTask.Recipient;
	RedeployStatus.State = FRedeployStatus::EState::RequestReceived;

	TArray<uint8> DecodedFileContent;
	FBase64::Decode(InRedeployListenerTask.FileContent, DecodedFileContent);

	FSHAHash Hash;
	FSHA1::HashBuffer(DecodedFileContent.GetData(), DecodedFileContent.Num(), Hash.Hash);
	if (Hash != InRedeployListenerTask.ExpectedHash)
	{
		const FString ErrorMsg = FString::Printf(TEXT("Incorrect checksum %s; expected %s"),
			*Hash.ToString(), *InRedeployListenerTask.ExpectedHash.ToString());
		UE_LOGF(LogSwitchboard, Error, "Redeploy listener: %ls", *ErrorMsg);
		SendMessage(CreateRedeployStatusMessage(InRedeployListenerTask.TaskID, false, ErrorMsg), InRedeployListenerTask.Recipient);
		return false;
	}

	UE_LOGF(LogSwitchboard, Display, "Successfully validated new listener checksum");

	const uint32 CurrentPid = FPlatformProcess::GetCurrentProcessId();

	// NOTE: FPlatformProcess::ExecutablePath() is stale if we were moved while running.
	const FString OriginalThisExePath = FPlatformProcess::GetApplicationName(CurrentPid);
	const FString ThisExeDir = FPaths::GetPath(OriginalThisExePath);
	const FString ThisExeExt = FPaths::GetExtension(OriginalThisExePath, true);

	const FString TempNewListenerPath = FPaths::CreateTempFilename(*ThisExeDir, TEXT("SwitchboardListener-New"), *ThisExeExt);

	UE_LOGF(LogSwitchboard, Display, "Writing new listener to \"%ls\" (%d bytes)", *TempNewListenerPath, DecodedFileContent.Num());
	if (!FFileHelper::SaveArrayToFile(DecodedFileContent, *TempNewListenerPath))
	{
		const FString ErrorMsg = FString::Printf(TEXT("Unable to write new listener to \"%s\""), *TempNewListenerPath);
		UE_LOGF(LogSwitchboard, Error, "Redeploy listener: %ls", *ErrorMsg);
		SendMessage(CreateRedeployStatusMessage(InRedeployListenerTask.TaskID, false, ErrorMsg), InRedeployListenerTask.Recipient);
		return false;
	}

	// Unbind the port so the redeployed child process can claim it.
	StopListening();

	// Create IPC semaphore.
	const FString IpcSemaphoreName = GetIpcSemaphoreName(CurrentPid);
	FPlatformProcess::FSemaphore* IpcSemaphore = FPlatformProcess::NewInterprocessSynchObject(IpcSemaphoreName, true);
	ON_SCOPE_EXIT
	{
		if (IpcSemaphore)
		{
			FPlatformProcess::DeleteInterprocessSynchObject(IpcSemaphore);
		}
	};

	if (IpcSemaphore == nullptr)
	{
		const FString ErrorMsg(TEXT("Unable to create IPC semaphore"));
		UE_LOGF(LogSwitchboard, Error, "Redeploy listener: %ls", *ErrorMsg);
		SendMessage(CreateRedeployStatusMessage(InRedeployListenerTask.TaskID, false, ErrorMsg), InRedeployListenerTask.Recipient);
		return false;
	}

	// We acquire to zero immediately; the child process will release/increment when initialized,
	// and we can await that indication by trying to acquire a second time.
	IpcSemaphore->Lock();

	RedeployStatus.OriginalThisExePath = OriginalThisExePath;
	RedeployStatus.TempNewListenerPath = TempNewListenerPath;
	RedeployStatus.State = FRedeployStatus::EState::NewListenerWrittenTemp;

	const FString RedeployArgs = Options.ToString() + FString::Printf(TEXT(" -RedeployFromPid=%u"), CurrentPid);
	const bool bLaunchDetached = true;
	const bool bLaunchHidden = false;
	const bool bLaunchReallyHidden = false;
	uint32* OutProcessId = nullptr;
	const int32 PriorityModifier = 0;
	const TCHAR* WorkingDirectory = nullptr;
	void* PipeWriteChild = nullptr;
	FProcHandle RedeployedProcHandle = FSwitchboardProcess::CreateProc(*TempNewListenerPath, *RedeployArgs, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, OutProcessId, PriorityModifier, WorkingDirectory, PipeWriteChild);
	if (!RedeployedProcHandle.IsValid())
	{
		const FString ErrorMsg(TEXT("Unable to launch new listener"));
		UE_LOGF(LogSwitchboard, Error, "Redeploy listener: %ls", *ErrorMsg);
		SendMessage(CreateRedeployStatusMessage(InRedeployListenerTask.TaskID, false, ErrorMsg), InRedeployListenerTask.Recipient);
		return false;
	}

	RedeployStatus.ListenerProc = RedeployedProcHandle;
	RedeployStatus.State = FRedeployStatus::EState::NewListenerStarted;

	const FString TempThisListenerPath = FPaths::CreateTempFilename(*ThisExeDir, TEXT("SwitchboardListener-Old"), *ThisExeExt);
	if (!IFileManager::Get().Move(*TempThisListenerPath, *OriginalThisExePath, false))
	{
		const uint32 LastError = FPlatformMisc::GetLastError();
		const FString ErrorMsg = FString::Printf(TEXT("Unable to move old listener to \"%s\" (error code %u)"), *TempThisListenerPath, LastError);
		UE_LOGF(LogSwitchboard, Error, "Redeploy listener: %ls", *ErrorMsg);
		SendMessage(CreateRedeployStatusMessage(InRedeployListenerTask.TaskID, false, ErrorMsg), InRedeployListenerTask.Recipient);
		return false;
	}

	RedeployStatus.TempThisListenerPath = TempThisListenerPath;
	RedeployStatus.State = FRedeployStatus::EState::ThisListenerRenamed;

	if (!IFileManager::Get().Move(*OriginalThisExePath, *TempNewListenerPath, false))
	{
		const uint32 LastError = FPlatformMisc::GetLastError();
		const FString ErrorMsg = FString::Printf(TEXT("Unable to move new listener to \"%s\" (error code %u)"), *TempThisListenerPath, LastError);
		UE_LOGF(LogSwitchboard, Error, "Redeploy listener: %ls", *ErrorMsg);
		SendMessage(CreateRedeployStatusMessage(InRedeployListenerTask.TaskID, false, ErrorMsg), InRedeployListenerTask.Recipient);
		return false;
	}

	RedeployStatus.State = FRedeployStatus::EState::NewListenerRenamed;

	const uint64 IpcReadyWaitSeconds = 5;
	const bool IpcAcquired = IpcSemaphore->TryLock(IpcReadyWaitSeconds * 1'000'000'000ULL);
	if (IpcAcquired == false)
	{
		const FString ErrorMsg(TEXT("Timed out waiting for child process to sync"));
		UE_LOGF(LogSwitchboard, Error, "Redeploy listener: %ls", *ErrorMsg);
		SendMessage(CreateRedeployStatusMessage(InRedeployListenerTask.TaskID, false, ErrorMsg), InRedeployListenerTask.Recipient);
		return false;
	}

	UE_LOGF(LogSwitchboard, Display, "Redeploy complete; shutting down");
	SendMessage(CreateRedeployStatusMessage(InRedeployListenerTask.TaskID, true, FString(TEXT("Redeployed instance is ready"))), InRedeployListenerTask.Recipient);

	RedeployStatus.State = FRedeployStatus::EState::Complete;
	RequestEngineExit(TEXT("Redeploy complete"));

	return true;
}

void FSwitchboardListener::RollbackRedeploy()
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::RollbackRedeploy);

	// Unwind redeploy actions, in reverse order of corresponding state transitions.
	UE_LOGF(LogSwitchboard, Warning, "Rolling back redeploy...");

	if ((uint8)RedeployStatus.State >= (uint8)FRedeployStatus::EState::NewListenerRenamed)
	{
		UE_LOGF(LogSwitchboard, Warning, "Moving new listener back to temporary location");
		IFileManager::Get().Move(*RedeployStatus.TempNewListenerPath, *RedeployStatus.OriginalThisExePath, false);
	}

	if ((uint8)RedeployStatus.State >= (uint8)FRedeployStatus::EState::ThisListenerRenamed)
	{
		UE_LOGF(LogSwitchboard, Warning, "Moving this listener back to original location");
		IFileManager::Get().Move(*RedeployStatus.OriginalThisExePath, *RedeployStatus.TempThisListenerPath, false);
		RedeployStatus.TempThisListenerPath.Reset();
	}

	if ((uint8)RedeployStatus.State >= (uint8)FRedeployStatus::EState::NewListenerStarted)
	{
		UE_LOGF(LogSwitchboard, Warning, "Terminating temporary listener");
		FPlatformProcess::TerminateProc(RedeployStatus.ListenerProc);
		FPlatformProcess::WaitForProc(RedeployStatus.ListenerProc);
		FPlatformProcess::CloseProc(RedeployStatus.ListenerProc);
	}

	if ((uint8)RedeployStatus.State >= (uint8)FRedeployStatus::EState::NewListenerWrittenTemp)
	{
		UE_LOGF(LogSwitchboard, Warning, "Removing temporary listener");
		IFileManager::Get().Delete(*RedeployStatus.TempNewListenerPath, true);
		RedeployStatus.OriginalThisExePath.Reset();
		RedeployStatus.TempNewListenerPath.Reset();

		// Resume listening on this instance.
		StartListening();
	}

	if ((uint8)RedeployStatus.State >= (uint8)FRedeployStatus::EState::RequestReceived)
	{
		RedeployStatus.RequestingClient = FIPv4Endpoint::Any;
		RedeployStatus.State = FRedeployStatus::EState::None;
	}
}

//static
FString FSwitchboardListener::GetIpcSemaphoreName(uint32 ParentPid)
{
	return FString::Printf(TEXT("SwitchboardListenerIpc_Redeploy_%u"), ParentPid);
}

bool FSwitchboardListener::Task_SendFileToClient(const FSwitchboardSendFileToClientTask& InSendFileToClientTask)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::Task_SendFileToClient);

	FString SourceFilePath = InSendFileToClientTask.Source;
	FPlatformMisc::NormalizePath(SourceFilePath);
	FPaths::MakePlatformFilename(SourceFilePath);

	if (!FPaths::FileExists(SourceFilePath))
	{
		const FString ErrorMsg = FString::Printf(TEXT("Could not find file %s"), *SourceFilePath);
		UE_LOGF(LogSwitchboard, Error, "%ls", *ErrorMsg);
		SendMessage(CreateSendFileToClientFailedMessage(InSendFileToClientTask.Source, ErrorMsg), InSendFileToClientTask.Recipient);
		return false;
	}

	TArray<uint8> FileContent;
	if (!FFileHelper::LoadFileToArray(FileContent, *SourceFilePath))
	{
		const FString ErrorMsg = FString::Printf(TEXT("Error reading from file %s"), *SourceFilePath);
		UE_LOGF(LogSwitchboard, Error, "%ls", *ErrorMsg);
		SendMessage(CreateSendFileToClientFailedMessage(InSendFileToClientTask.Source, ErrorMsg), InSendFileToClientTask.Recipient);
		return false;
	}

	const FString EncodedFileContent = FBase64::Encode(FileContent);
	return SendMessage(CreateSendFileToClientCompletedMessage(InSendFileToClientTask.Source, EncodedFileContent), InSendFileToClientTask.Recipient);
}


void FSwitchboardListener::FillStatePacket(FSwitchboardStatePacket& OutStatePacket)
{
	FPlatformMisc::GetOSVersions(OutStatePacket.OsVersionLabel, OutStatePacket.OsVersionLabelSub);
	OutStatePacket.OsVersionNumber = FPlatformMisc::GetOSVersion();
	OutStatePacket.bProcessorSMT = bProcessorSMT;
	OutStatePacket.TotalPhysicalMemory = FPlatformMemory::GetConstants().TotalPhysical;
	OutStatePacket.PlatformBinaryDirectory = FPlatformProcess::GetBinariesSubdirectory();

	OutStatePacket.RunningProcesses.Empty(RunningProcesses.Num());
	for (const TSharedPtr<FRunningProcess>& RunningProcess : RunningProcesses)
	{
		check(RunningProcess.IsValid());

		FSwitchboardStateRunningProcess StateRunningProcess;

		StateRunningProcess.Uuid = RunningProcess->UUID.ToString();
		StateRunningProcess.Name = RunningProcess->Name;
		StateRunningProcess.Path = RunningProcess->Path;
		StateRunningProcess.Caller = RunningProcess->Caller;
		StateRunningProcess.Pid = RunningProcess->PID;

		OutStatePacket.RunningProcesses.Add(MoveTemp(StateRunningProcess));
	}
}


//static
_Function_class_(QUIC_LISTENER_CALLBACK)
QUIC_STATUS QUIC_API FSwitchboardListener::QuicListenerThunk(HQUIC Listener, void* Context, QUIC_LISTENER_EVENT* Event)
{
	FSwitchboardListener* This = reinterpret_cast<FSwitchboardListener*>(Context);
	return This->QuicListenerCallback(Listener, Event);
}


QUIC_STATUS FSwitchboardListener::QuicListenerCallback(HQUIC Listener, QUIC_LISTENER_EVENT* Event)
{
	switch (Event->Type) {
		case QUIC_LISTENER_EVENT_NEW_CONNECTION:
		{
			// A new connection is being attempted by a client. For the handshake to
			// proceed, the server must provide a configuration for QUIC to use. The
			// app MUST set the callback handler before returning.
			const FIPv4Endpoint Endpoint = EndpointFromQuicAddr(*Event->NEW_CONNECTION.Info->RemoteAddress);
			UE_LOGFMT(LogSwitchboard, Verbose, "[quic][{Endpoint}] LISTENER_EVENT_NEW_CONNECTION", Endpoint.ToString());

			FConnectionRef Connection = MakeShared<FConnection>();
			Connection->Endpoint = Endpoint;
			Connection->QuicConn = Event->NEW_CONNECTION.Connection;
			Connection->InactiveTimeout = DefaultInactiveTimeoutSeconds;
			Connection->LastActivityTime = FPlatformTime::Seconds();

			{
				UE::TUniqueLock<UE::FRecursiveMutex> ConnectionsScopeLock(ConnectionsLock);
				ConnectionsByEndpoint.Add(Endpoint, Connection);
				ConnectionsByQuicConn.Add(Connection->QuicConn, Connection);
			}

			QuicApi->SetCallbackHandler(Connection->QuicConn, reinterpret_cast<void*>(&QuicConnectionThunk), this);
			QuicApi->ConnectionSetConfiguration(Connection->QuicConn, QuicConfiguration);

			return QUIC_STATUS_SUCCESS;
		}
		default:
			break;
	}

	return QUIC_STATUS_NOT_SUPPORTED;
}


//static
QUIC_STATUS QUIC_API FSwitchboardListener::QuicConnectionThunk(HQUIC Connection, void* Context, QUIC_CONNECTION_EVENT* Event)
{
	FSwitchboardListener* This = reinterpret_cast<FSwitchboardListener*>(Context);
	return This->QuicConnectionCallback(Connection, Event);
}


QUIC_STATUS FSwitchboardListener::QuicConnectionCallback(HQUIC QuicConn, QUIC_CONNECTION_EVENT* Event)
{
	QUIC_ADDR RemoteAddr;
	uint32_t BufferSize = sizeof(RemoteAddr);
	QUIC_ENSURE(QuicApi->GetParam(QuicConn, QUIC_PARAM_CONN_REMOTE_ADDRESS, &BufferSize, &RemoteAddr));
	const FIPv4Endpoint RemoteEndpoint = EndpointFromQuicAddr(RemoteAddr);

	switch (Event->Type)
	{
		case QUIC_CONNECTION_EVENT_PEER_CERTIFICATE_RECEIVED:
			// TODO, someday: Client certificate auth. //return QUIC_STATUS_UNKNOWN_CERTIFICATE;
			break;
		case QUIC_CONNECTION_EVENT_CONNECTED:
			// The handshake has completed for the connection.
			UE_LOGFMT(LogSwitchboard, Display, "[quic][conn][{Endpoint}] Connected", *RemoteEndpoint.ToString());
			QuicApi->ConnectionSendResumptionTicket(QuicConn, QUIC_SEND_RESUMPTION_FLAG_NONE, 0, NULL);
			break;
		case QUIC_CONNECTION_EVENT_SHUTDOWN_INITIATED_BY_TRANSPORT:
			// The connection has been shut down by the transport. Generally, this
			// is the expected way for the connection to shut down with this
			// protocol, since we let idle timeout kill the connection.
			if (Event->SHUTDOWN_INITIATED_BY_TRANSPORT.Status == QUIC_STATUS_CONNECTION_IDLE)
			{
				UE_LOGFMT(LogSwitchboard, Display, "[quic][conn][{Endpoint}] Successfully shut down on idle.",
					*RemoteEndpoint.ToString());
			}
			else
			{
				UE_LOGFMT(LogSwitchboard, Display, "[quic][conn][{Endpoint}] Shut down by transport, {Status}",
					*RemoteEndpoint.ToString(), static_cast<int64>(Event->SHUTDOWN_INITIATED_BY_TRANSPORT.Status));
			}
			break;
		case QUIC_CONNECTION_EVENT_SHUTDOWN_INITIATED_BY_PEER:
			// The connection was explicitly shut down by the peer.
			UE_LOGFMT(LogSwitchboard, Display, "[quic][conn][{Endpoint}] Shut down by peer, {ErrorCode}",
				*RemoteEndpoint.ToString(), static_cast<uint64>(Event->SHUTDOWN_INITIATED_BY_PEER.ErrorCode));
			break;
		case QUIC_CONNECTION_EVENT_SHUTDOWN_COMPLETE:
		{
			// The connection has completed the shutdown process and is ready to be
			// safely cleaned up.
			UE_LOGFMT(LogSwitchboard, Display, "[quic][conn][{Endpoint}] Shutdown complete", *RemoteEndpoint.ToString());

			{
				UE::TUniqueLock<UE::FRecursiveMutex> ConnectionsScopeLock(ConnectionsLock);
				FConnectionRef Connection = ConnectionsByQuicConn.FindAndRemoveChecked(QuicConn);
				ConnectionsByEndpoint.FindAndRemoveChecked(Connection->Endpoint);
				if (Connection->QuicStream)
				{
					ConnectionsByQuicStream.FindAndRemoveChecked(Connection->QuicStream);
				}
			}

			QuicApi->ConnectionClose(QuicConn);
			break;
		}
		case QUIC_CONNECTION_EVENT_PEER_STREAM_STARTED:
		{
			// The peer has started/created a new stream. The app MUST set the
			// callback handler before returning.
			UE_LOGFMT(LogSwitchboard, Display, "[quic][conn][{Endpoint}] Peer stream started", *RemoteEndpoint.ToString());

			UE::TUniqueLock<UE::FRecursiveMutex> ConnectionsScopeLock(ConnectionsLock);

			FConnectionRef Connection = ConnectionsByQuicConn[QuicConn];
			ensure(Connection->QuicStream == nullptr); // Single stream per connection
			Connection->QuicStream = Event->PEER_STREAM_STARTED.Stream;

			ConnectionsByQuicStream.Add(Connection->QuicStream, Connection);

			QuicApi->SetCallbackHandler(Connection->QuicStream, reinterpret_cast<void*>(&QuicStreamThunk), this);

			break;
		}
		case QUIC_CONNECTION_EVENT_RESUMED:
			// The connection succeeded in doing a TLS resumption of a previous
			// connection's session.
			UE_LOGFMT(LogSwitchboard, Display, "[quic][conn][{Endpoint}] Connection resumed", *RemoteEndpoint.ToString());
			break;
		default:
			break;
	}

	return QUIC_STATUS_SUCCESS;
}


//static
QUIC_STATUS QUIC_API FSwitchboardListener::QuicStreamThunk(HQUIC Stream, void* Context, QUIC_STREAM_EVENT* Event)
{
	FSwitchboardListener* This = reinterpret_cast<FSwitchboardListener*>(Context);
	return This->QuicStreamCallback(Stream, Event);
}


QUIC_STATUS FSwitchboardListener::QuicStreamCallback(HQUIC Stream, QUIC_STREAM_EVENT* Event)
{
	UE::TUniqueLock<UE::FRecursiveMutex> ConnectionsScopeLock(ConnectionsLock);
	FConnectionRef Connection = ConnectionsByQuicStream[Stream];

	QUIC_ADDR RemoteAddr;
	uint32_t BufferSize = sizeof(RemoteAddr);
	QUIC_ENSURE(QuicApi->GetParam(Connection->QuicConn, QUIC_PARAM_CONN_REMOTE_ADDRESS, &BufferSize, &RemoteAddr));
	const FIPv4Endpoint RemoteEndpoint = EndpointFromQuicAddr(RemoteAddr);

	QUIC_UINT62 StreamId;
	BufferSize = sizeof(StreamId);
	QUIC_ENSURE(QuicApi->GetParam(Stream, QUIC_PARAM_STREAM_ID, &BufferSize, &StreamId));

	const FString StreamStr = FString::Printf(TEXT("%s[%llu]"), *RemoteEndpoint.ToString(), StreamId);

	switch (Event->Type)
	{
		case QUIC_STREAM_EVENT_SEND_COMPLETE:
		{
			// A previous StreamSend call has completed, and the context is being
			// returned back to the app.
			UE::TUniqueLock<UE::FMutex> SendLock(Connection->SendLock);
			TSharedPtr<FQuicBuffer> DequeuedSend;
			Connection->SendBuffers.Dequeue(DequeuedSend);

			// Single stream, ordered delivery, dequeue should release the right thing
			ensure(DequeuedSend.Get() == Event->SEND_COMPLETE.ClientContext);

			break;
		}
		case QUIC_STREAM_EVENT_RECEIVE:
		{
			// The value of this struct member when the callback returns is taken to mean
			// how many bytes we actually consumed. Bytes left unconsumed will be handled
			// after the tick re-enables receiving, upon consuming the preceding message.
			Event->RECEIVE.TotalBufferLength = 0;

			// Data was received from the peer on the stream.
			Connection->LastActivityTime = FPlatformTime::Seconds();

			UE::TUniqueLock<UE::FMutex> ReceiveLock(Connection->ReceiveLock);

			if (!Connection->bMessageComplete)
			{
				for (uint32_t BufferNum = 0; BufferNum < Event->RECEIVE.BufferCount; ++BufferNum)
				{
					const QUIC_BUFFER Buffer = Event->RECEIVE.Buffers[BufferNum];
					for (uint32 ReadIdx = 0; ReadIdx < Buffer.Length; ++ReadIdx)
					{
						const uint8_t Byte = Buffer.Buffer[ReadIdx];
						Connection->ReceiveBuffer->Add(Byte);
						++Event->RECEIVE.TotalBufferLength;

						// If this concluded the outstanding message, signal the main thread to
						// process it, and let MsQuic keep the next message buffered internally.
						// Partial buffer consumption implicitly suspends receive events, which
						// we then resume after the main thread processes the previous message.
						if (Byte == 0)
						{
							Connection->bMessageComplete = true;

							// Break inner loop + outer loop + switch
							goto stream_receive_outer_break;
						}
					}
				}
			}

stream_receive_outer_break:
			break;
		}
		case QUIC_STREAM_EVENT_PEER_SEND_SHUTDOWN:
			// The peer gracefully shut down its send direction of the stream.
			UE_LOGFMT(LogSwitchboard, Display, "[quic][strm][{Stream}] Peer shut down", StreamStr);
			break;
		case QUIC_STREAM_EVENT_PEER_SEND_ABORTED:
			// The peer aborted its send direction of the stream.
			UE_LOGFMT(LogSwitchboard, Display, "[quic][strm][{Stream}] Peer aborted", StreamStr);
			QuicApi->StreamShutdown(Stream, QUIC_STREAM_SHUTDOWN_FLAG_ABORT, 0);
			break;
		case QUIC_STREAM_EVENT_SHUTDOWN_COMPLETE:
		{
			// Both directions of the stream have been shut down and MsQuic is done
			// with the stream. It can now be safely cleaned up.
			UE_LOGFMT(LogSwitchboard, Display, "[quic][strm][{Stream}] Shutdown complete", StreamStr);

			ensure(Connection->QuicStream == Stream);
			ConnectionsByQuicStream.FindAndRemoveChecked(Stream);
			QuicApi->StreamClose(Stream);
			Connection->QuicStream = nullptr;

			break;
		}
		default:
			break;
	}

	return QUIC_STATUS_SUCCESS;
}


#if PLATFORM_WINDOWS
static void FillOutTaskbarAutoHide(FSyncStatus& SyncStatus)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FillOutTaskbarAutoHide);

	APPBARDATA AppBarData;

	AppBarData.cbSize = sizeof(APPBARDATA);
	AppBarData.hWnd = nullptr;
	
	const UINT Result = UINT(SHAppBarMessage(ABM_GETSTATE, &AppBarData));
	
	if (Result == ABS_AUTOHIDE)
	{
		SyncStatus.Taskbar = TEXT("AutoHide");
	}
	else
	{
		SyncStatus.Taskbar = TEXT("OnTop");
	}
}
#endif // PLATFORM_WINDOWS


#if PLATFORM_WINDOWS
void FSwitchboardListener::HarvestExistingFlipModeSessions()
{
	RunningFlipModeSessions.Reserve(MaxFlipModeMonitorSessions);
	RunningFlipModeSessions.Reset();

	for (FFlipModeMonitorSessionId SessionId = 0; SessionId < MaxFlipModeMonitorSessions; ++SessionId)
	{
		bool bOutStoppedSession = false;
		if (!EnsureFlipModeSessionAvailable(SessionId, &bOutStoppedSession))
		{
			UE_LOGFMT(LogSwitchboard, Error, "Unable to stop existing flip mode session {SessionId}", SessionId);
			RunningFlipModeSessions.Add(true, 1);
		}
		else
		{
			if (bOutStoppedSession)
			{
				UE_LOGFMT(LogSwitchboard, Warning, "Stopped existing flip mode session {SessionId}", SessionId);
			}

			RunningFlipModeSessions.Add(false, 1);
		}
	}
}


bool FSwitchboardListener::EnsureFlipModeSessionAvailable(FFlipModeMonitorSessionId SessionId, bool* bOutStoppedSession)
{
	const FString SessionName = MakeFlipModeMonitorSessionName(SessionId);

	EVENT_TRACE_PROPERTIES TraceProps;
	FMemory::Memzero(TraceProps);
	TraceProps.Wnode.BufferSize = sizeof(TraceProps);
	TraceProps.Wnode.Flags = WNODE_FLAG_TRACED_GUID;

	if (bOutStoppedSession != nullptr)
	{
		*bOutStoppedSession = false;
	}

	const uint32 Result = ControlTrace(0, *SessionName, &TraceProps, EVENT_TRACE_CONTROL_STOP);
	if (Result == ERROR_SUCCESS || Result == ERROR_MORE_DATA)
	{
		if (bOutStoppedSession != nullptr)
		{
			*bOutStoppedSession = true;
		}
		return true;
	}
	else if (Result == ERROR_WMI_INSTANCE_NOT_FOUND)
	{
		// Didn't exist.
		return true;
	}

	UE_LOGFMT(LogSwitchboard, Error, "Failed to stop flip mode session {SessionName} with result {Result}",
		SessionName, Result);

	return false;
}


FSwitchboardListener::FFlipModeMonitorSessionId FSwitchboardListener::AllocFlipModeMonitorSessionId()
{
	check(IsInGameThread());

	const int32 AvailableId = RunningFlipModeSessions.FindAndSetFirstZeroBit();
	if (AvailableId < 0)
	{
		UE_LOGFMT(LogSwitchboard, Error, "Failed to allocate flip mode monitor session ID; too many sessions");
		return InvalidFlipModeSessionId;
	}

	UE_LOGFMT(LogSwitchboard, Verbose, "Allocated flip mode monitor session ID {ID}", AvailableId);

	return static_cast<FFlipModeMonitorSessionId>(AvailableId);
}


void FSwitchboardListener::FreeFlipModeMonitorSessionId(FFlipModeMonitorSessionId SessionId)
{
	check(IsInGameThread());

	if (!ensure(SessionId < RunningFlipModeSessions.Num() && SessionId < MaxFlipModeMonitorSessions))
	{
		UE_LOGFMT(LogSwitchboard, Error, "Attempted to free invalid flip mode monitor session ID {Id}", SessionId);
		return;
	}

	if (!ensure(RunningFlipModeSessions[SessionId]))
	{
		UE_LOGFMT(LogSwitchboard, Error, "Attempted to free unused flip mode monitor session ID {Id}", SessionId);
		return;
	}

	RunningFlipModeSessions[SessionId] = false;

	UE_LOGFMT(LogSwitchboard, Verbose, "Freed flip mode monitor session ID {ID}", SessionId);
}


FString FSwitchboardListener::MakeFlipModeMonitorSessionName(FFlipModeMonitorSessionId SessionId)
{
	return FString::Printf(TEXT("switchboard_%s_presentmon_%d"),
		 *ListenerEndpoint->ToString(), SessionId);
}


FRunningProcess* FSwitchboardListener::FindOrStartFlipModeMonitorForUUID(const FGuid& UUID)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::FindOrStartFlipModeMonitorForUUID);

	// See if the associated FlipModeMonitor is running
	{
		TSharedPtr<FRunningProcess>* FlipModeMonitorPtr = FlipModeMonitors.FindByPredicate([&](const TSharedPtr<FRunningProcess>& FlipMonitor)
		{
			check(FlipMonitor.IsValid());
			return FlipMonitor->UUID == UUID;
		});

		if (FlipModeMonitorPtr)
		{
			return (*FlipModeMonitorPtr).Get();
		}
	}

	// It wasn't in there, so let's find our target process

	TSharedPtr<FRunningProcess> Process;
	{
		TSharedPtr<FRunningProcess>* ProcessPtr = RunningProcesses.FindByPredicate([&](const TSharedPtr<FRunningProcess>& InProcess)
		{
			check(InProcess.IsValid());
			return InProcess->UUID == UUID;
		});

		if (ProcessPtr)
		{
			Process = *ProcessPtr;
		}
	}

	// If the target process does not exist, no point in continuing
	if (!Process.IsValid())
	{
		return nullptr;
	}

	// Ok, we need to create our monitor.
	TSharedRef<FRunningProcess> MonitorProcess = MakeShared<FRunningProcess>();

	if (!MonitorProcess->CreatePipes())
	{
		UE_LOGF(LogSwitchboard, Error, "Could not create pipe to read MonitorProcess output!");
		return nullptr;
	}

	const bool bLaunchDetached = true;
	const bool bLaunchHidden = false;
	const bool bLaunchReallyHidden = false;
	const int32 PriorityModifier = 0;
	const TCHAR* WorkingDirectory = nullptr;

	MonitorProcess->Path = FPaths::EngineDir() / TEXT("Binaries") / TEXT("ThirdParty") / TEXT("PresentMon") / TEXT("Win64") / TEXT("PresentMon-1.8.0-x64.exe");

	const FFlipModeMonitorSessionId SessionId = AllocFlipModeMonitorSessionId();
	if (SessionId == InvalidFlipModeSessionId)
	{
		UE_LOGFMT(LogSwitchboard, Error, "Failed to allocate flip mode monitor session");
		return nullptr;
	}

	MonitorProcess->OnProcessEnded.AddLambda(
		[this, SessionId]()
		{
			if (EnsureFlipModeSessionAvailable(SessionId))
			{
				FreeFlipModeMonitorSessionId(SessionId);
			}
			else
			{
				UE_LOGFMT(LogSwitchboard, Error, "Unable to recover flip mode monitor session {SessionId}", SessionId);
			}
		}
	);

	const FString SessionName = MakeFlipModeMonitorSessionName(SessionId);

	const FString Arguments = FString::Printf(
		TEXT("-session_name %s -output_stdout -terminate_on_proc_exit -stop_existing_session -process_id %d"), 
		*SessionName, Process->PID);

	MonitorProcess->Handle = FSwitchboardProcess::CreateProc(
		*MonitorProcess->Path,
		*Arguments,
		bLaunchDetached,
		bLaunchHidden,
		bLaunchReallyHidden,
		&MonitorProcess->PID,
		PriorityModifier,
		WorkingDirectory,
		MonitorProcess->StdoutChildWritePipe,
		MonitorProcess->StdinChildReadPipe
	);

	if (!MonitorProcess->Handle.IsValid() || !FPlatformProcess::IsProcRunning(MonitorProcess->Handle))
	{
		// Close process in case it just didn't run
		FPlatformProcess::CloseProc(MonitorProcess->Handle);

		// Close unused pipes
		MonitorProcess->ClosePipes();

		// Log error
		const FString ErrorMsg = FString::Printf(TEXT("Could not start FlipMode monitor %s"), *MonitorProcess->Path);
		UE_LOGF(LogSwitchboard, Error, "%ls", *ErrorMsg);

		FreeFlipModeMonitorSessionId(SessionId);
		return nullptr;
	}

	// Log success
	UE_LOGF(LogSwitchboard, Display, "Started FlipMode monitor %d: %ls %ls", MonitorProcess->PID, *MonitorProcess->Path, *Arguments);

	// The UUID corresponds to the program being monitored. This will be used when looking for the Monitor of a given process.
	// The monitor auto-closes when monitored program closes.
	MonitorProcess->UUID = Process->UUID;
	MonitorProcess->bUpdateClientsWithStdout = false;
	MonitorProcess->bLockGpuClock = false;
	MonitorProcess->Recipient = InvalidEndpoint;
	MonitorProcess->Name = TEXT("flipmode_monitor");

	FlipModeMonitors.Add(MonitorProcess);

	return &MonitorProcess.Get();
}
#endif // PLATFORM_WINDOWS

#if PLATFORM_WINDOWS
static void FillOutFlipMode(FSyncStatus& SyncStatus, FRunningProcess* FlipModeMonitor)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FillOutFlipMode);

	// See if the flip monitor is still there.
	if (!FlipModeMonitor || !FlipModeMonitor->Handle.IsValid())
	{
		SyncStatus.FlipModeHistory.Add(TEXT("n/a")); // this informs Switchboard that data is not valid
		return;
	}

	// Get stdout.
	FUTF8ToTCHAR StdoutConv(reinterpret_cast<const ANSICHAR*>(FlipModeMonitor->Stdout.GetData()), FlipModeMonitor->Stdout.Num());
	const FString StdOut(StdoutConv.Length(), StdoutConv.Get());

	// Clear out the StdOut array.
	FlipModeMonitor->Stdout.Empty();

	// Split into lines

	TArray<FString> Lines;
	StdOut.ParseIntoArrayLines(Lines, false);

	// Interpret the output as follows:
	//
	// Application,ProcessID,SwapChainAddress,Runtime,
	// SyncInterval,PresentFlags,AllowsTearing,
	// TimeInSeconds,MsBetweenPresents,MsBetweenDisplayChange,Dropped,
	// PresentMode,
	// MsInPresentAPI,MsUntilRenderComplete,MsUntilDisplayed
	//
	// e.g.
	//   "UnrealEditor.exe,23256,0x000002DFBFE603A0,DXGI,1,0  ,0,35.65057220000000,1.91800000000000,30735.98660000000018,0,Composed: Flip,1.91820000000000,32.67810000000000,30700.80250000000160"

	TArray<FString> Fields;

	for (const FString& Line : Lines)
	{
		Line.ParseIntoArray(Fields, TEXT(","), false);
		
		if (Fields.Num() != 15)
		{
			continue;
		}

		const int32 PresentMonIdx = 11;

		// Skip "PresentMode", which is not a real flip mode.
		if (Fields[PresentMonIdx] == TEXT("PresentMode"))
		{
			continue;
		}

		// Only add if it's different from the last entry
		if ((SyncStatus.FlipModeHistory.Num() == 0) || (SyncStatus.FlipModeHistory.Last() != Fields[PresentMonIdx]))
		{
			SyncStatus.FlipModeHistory.Add(Fields[PresentMonIdx]);
		}
	}
}
#endif // PLATFORM_WINDOWS


#if PLATFORM_WINDOWS
static void FillOutDisableFullscreenOptimizationForProcess(FSyncStatus& SyncStatus, const FRunningProcess* Process)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FillOutDisableFullscreenOptimizationForProcess);

	// Reset output array just in case
	SyncStatus.ProgramLayers.Reset();

	// No point in continuing if there is no process to get the flags for.
	if (!Process)
	{
		return;
	}

	// This is the absolute path of the program we'll be looking for in the registry
	const FString ProcessAbsolutePath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*Process->Path);

	const FString LayersKeyPath = TEXT("Software\\Microsoft\\Windows NT\\CurrentVersion\\AppCompatFlags\\Layers");

	// Check if the key exists
	HKEY LayersKey;
	if (RegOpenKeyExW(HKEY_CURRENT_USER, *LayersKeyPath, 0, KEY_READ, &LayersKey))
	{
		return;
	}

	// If the key exists, the Value Names are the paths to the programs
	const TArray<FString> ProgramPaths = RegistryGetValueNames(LayersKey);
	const FString* ExistingPath = Algo::FindBy(ProgramPaths, ProcessAbsolutePath, [](const FString& ProgPath) { return IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ProgPath); });
	if (ExistingPath)
	{
		// If so, get the layers from the Value Data.
		const FString ProgramLayers = RegistryGetStringValueData(LayersKey, *ExistingPath);
		ProgramLayers.ParseIntoArray(SyncStatus.ProgramLayers, TEXT(" "));
	}

	RegCloseKey(LayersKey);
}
#endif // PLATFORM_WINDOWS


bool FSwitchboardListener::EquivalentTaskFutureExists(uint32 TaskEquivalenceHash) const
{
	return !!MessagesFutures.FindByPredicate([=](const FSwitchboardMessageFuture& MessageFuture)
	{
		return MessageFuture.EquivalenceHash == TaskEquivalenceHash;
	});
}

bool FSwitchboardListener::Task_GetSyncStatus(const FSwitchboardGetSyncStatusTask& InGetSyncStatusTask)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::Task_GetSyncStatus);

#if PLATFORM_WINDOWS
	// Reject request if an equivalent one is already in our future
	if (EquivalentTaskFutureExists(InGetSyncStatusTask.GetEquivalenceHash()))
	{
		SendMessage(
			CreateTaskDeclinedMessage(InGetSyncStatusTask, TEXT("Duplicate"),{}),
			InGetSyncStatusTask.Recipient
		);

		return false;
	}

	TSharedRef<FSyncStatus> SyncStatus = MakeShared<FSyncStatus>(); // Smart pointer to avoid potentially bigger copy to lambda below.

	// We need to run these on this thread to avoid threading issues.
	if (EnumHasAnyFlags(InGetSyncStatusTask.RequestFlags, ESyncStatusRequestFlags::FlipModeHistory))
	{
		FillOutFlipMode(SyncStatus.Get(), FindOrStartFlipModeMonitorForUUID(InGetSyncStatusTask.ProgramID));
	}

	// Fill out fullscreen optimization setting
	if (EnumHasAnyFlags(InGetSyncStatusTask.RequestFlags, ESyncStatusRequestFlags::ProgramLayers))
	{
		TSharedPtr<FRunningProcess>* ProcessPtr = RunningProcesses.FindByPredicate([&](const TSharedPtr<FRunningProcess>& InProcess)
		{
			check(InProcess.IsValid());
			return !InProcess->bPendingKill && (InProcess->UUID == InGetSyncStatusTask.ProgramID);
		});

		FRunningProcess* Process = nullptr;

		if (ProcessPtr)
		{
			Process = (*ProcessPtr).Get();
		}

		FillOutDisableFullscreenOptimizationForProcess(SyncStatus.Get(), Process);
	}

	// Create our future message
	FSwitchboardMessageFuture MessageFuture;

	MessageFuture.InEndpoint = InGetSyncStatusTask.Recipient;
	MessageFuture.EquivalenceHash = InGetSyncStatusTask.GetEquivalenceHash();

	MessageFuture.Future = Async(EAsyncExecution::ThreadPool,
		[
			SyncStatus,
			RequestFlags = InGetSyncStatusTask.RequestFlags,
			CpuMonitor = CpuMonitor,
			CachedMosaicToposLock = CachedMosaicToposLock,
			CachedMosaicTopos = CachedMosaicTopos
		]() {
			SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE_STR("FSwitchboardListener::Task_GetSyncStatus future closure");

			// Load NvApi with FScopedNvApi only if we have requests that need it.
			ESyncStatusRequestFlags FlagsThatNeedNvApi = ESyncStatusRequestFlags::DriverInfo;
			EnumAddFlags(FlagsThatNeedNvApi, ESyncStatusRequestFlags::SyncTopos);

			if (EnumHasAnyFlags(RequestFlags, FlagsThatNeedNvApi))
			{
				FScopedNvApi ScopedNvApi;

				if (EnumHasAnyFlags(RequestFlags, ESyncStatusRequestFlags::DriverInfo))
				{
					ScopedNvApi.FillOutDriverVersion(SyncStatus.Get());
				}

				if (EnumHasAnyFlags(RequestFlags, ESyncStatusRequestFlags::SyncTopos))
				{
					ScopedNvApi.FillOutSyncTopologies(SyncStatus->SyncTopos);
				}
			}

			if (EnumHasAnyFlags(RequestFlags, ESyncStatusRequestFlags::MosaicTopos))
			{
				FReadScopeLock Lock(*CachedMosaicToposLock);
				SyncStatus->MosaicTopos = *CachedMosaicTopos;
			}

			if (EnumHasAnyFlags(RequestFlags, ESyncStatusRequestFlags::Taskbar))
			{
				FillOutTaskbarAutoHide(SyncStatus.Get());
			}

			if (EnumHasAnyFlags(RequestFlags, ESyncStatusRequestFlags::PidInFocus))
			{
				SyncStatus->PidInFocus = FindPidInFocus();
			}

			if (CpuMonitor && EnumHasAnyFlags(RequestFlags, ESyncStatusRequestFlags::CpuUtilization))
			{
				CpuMonitor->GetPerCoreUtilization(SyncStatus->CpuUtilization);
			}

			if (EnumHasAnyFlags(RequestFlags, ESyncStatusRequestFlags::AvailablePhysicalMemory))
			{
				const FPlatformMemoryStats MemStats = FPlatformMemory::GetStats();
				SyncStatus->AvailablePhysicalMemory = MemStats.AvailablePhysical;
			}
			
			// Query GPU stats. These use NVML
			{
				const bool bGetUtilizations = EnumHasAnyFlags(RequestFlags, ESyncStatusRequestFlags::GpuUtilization);
				const bool bGetClocks = EnumHasAnyFlags(RequestFlags, ESyncStatusRequestFlags::GpuCoreClockKhz);
				const bool bGetTemperatures = EnumHasAnyFlags(RequestFlags, ESyncStatusRequestFlags::GpuTemperature);

				if (bGetUtilizations || bGetClocks || bGetTemperatures)
				{
					FScopedNvApi ScopedNvApi;
					ScopedNvApi.FillOutPhysicalGpuStats(SyncStatus.Get(), bGetUtilizations, bGetClocks, bGetTemperatures);
				}
			}

			return CreateSyncStatusMessage(SyncStatus.Get(), RequestFlags);
		}
	);

	// Queue it to be sent when ready
	MessagesFutures.Emplace(MoveTemp(MessageFuture));

	return true;
#else
	SendMessage(CreateTaskDeclinedMessage(InGetSyncStatusTask, "Platform not supported", {}), InGetSyncStatusTask.Recipient);
	return false;
#endif // PLATFORM_WINDOWS
}

bool FSwitchboardListener::Task_RefreshMosaics(const FSwitchboardRefreshMosaicsTask& InRefreshMosaicsTask)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::Task_RefreshMosaics);

#if PLATFORM_WINDOWS
	{
		FScopedNvApi ScopedNvApi;

		if (!ScopedNvApi.IsNvApiInitialized())
		{
			SendMessage(
				CreateTaskDeclinedMessage(InRefreshMosaicsTask, "NvAPI not supported", {}),
				InRefreshMosaicsTask.Recipient);
			return false;
		}
	}

	// Reject request if an equivalent one is already in our future
	if (EquivalentTaskFutureExists(InRefreshMosaicsTask.GetEquivalenceHash()))
	{
		SendMessage(
			CreateTaskDeclinedMessage(InRefreshMosaicsTask, TEXT("Duplicate"), {}),
			InRefreshMosaicsTask.Recipient
		);

		return false;
	}

	// Create our future
	FSwitchboardMessageFuture TaskFuture;
	TaskFuture.EquivalenceHash = InRefreshMosaicsTask.GetEquivalenceHash();
	TaskFuture.Future = Async(EAsyncExecution::ThreadPool,
		[
			CachedMosaicToposLock=CachedMosaicToposLock,
			CachedMosaicTopos=CachedMosaicTopos
		]()
		{
			SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE_STR("FSwitchboardListener::Task_RefreshMosaics future closure");

			FWriteScopeLock Lock(*CachedMosaicToposLock);
			CachedMosaicTopos->Reset();

			FScopedNvApi ScopedNvApi;
			ScopedNvApi.FillOutMosaicTopologies(*CachedMosaicTopos);
			
			return FString();
		}
	);

	MessagesFutures.Emplace(MoveTemp(TaskFuture));

	return true;
#else
	SendMessage(CreateTaskDeclinedMessage(InRefreshMosaicsTask, "Platform not supported", {}), InRefreshMosaicsTask.Recipient);
	return false;
#endif // PLATFORM_WINDOWS
}

bool FSwitchboardListener::Task_MinimizeWindows(const FSwitchboardMinimizeWindowsTask& InTask)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::Task_MinimizeWindows);

#if PLATFORM_WINDOWS
	// Reject request if an equivalent one is already in our future
	if (EquivalentTaskFutureExists(InTask.GetEquivalenceHash()))
	{
		SendMessage(
			CreateTaskDeclinedMessage(InTask, TEXT("Duplicate"), {}),
			InTask.Recipient
		);

		return false;
	}

	// Create our future
	FSwitchboardMessageFuture TaskFuture;
	TaskFuture.EquivalenceHash = InTask.GetEquivalenceHash();
	TaskFuture.Future = Async(EAsyncExecution::ThreadPool,
		[]()
		{
			SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE_STR("FSwitchboardListener::Task_MinimizeWindows future closure");

			if (const HWND HWnd = ::FindWindowA("Shell_TrayWnd", NULL))
			{
				::SendMessageA(HWnd, WM_COMMAND, (WPARAM)419, 0);
			}

			return FString();
		}
	);

	MessagesFutures.Emplace(MoveTemp(TaskFuture));

	return true;
#else
	SendMessage(CreateTaskDeclinedMessage(InTask, "Platform not supported", {}), InTask.Recipient);
	return false;
#endif // PLATFORM_WINDOWS
}

bool FSwitchboardListener::Task_SetInactiveTimeout(const FSwitchboardSetInactiveTimeoutTask& InTimeoutTask)
{
	const FIPv4Endpoint& Client = InTimeoutTask.Recipient;
	const float RequestedTimeout = InTimeoutTask.TimeoutSeconds;

	if (RequestedTimeout < DefaultInactiveTimeoutSeconds)
	{
		SendMessage(CreateTaskDeclinedMessage(InTimeoutTask, "Requested timeout too low", {}), Client);
		return false;
	}

	float& ClientTimeout = ConnectionsByEndpoint[Client]->InactiveTimeout;
	if (RequestedTimeout != ClientTimeout)
	{
		UE_LOGF(LogSwitchboard, Display, "Changing client %ls inactive timeout from %.0f to %.0f seconds",
			*Client.ToString(), ClientTimeout, RequestedTimeout);
		ClientTimeout = RequestedTimeout;
	}

	return true;
}

bool FSwitchboardListener::Task_FreeListenerBinary(const FSwitchboardFreeListenerBinaryTask& InFreeListenerBinaryTask)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::Task_FreeListenerBinary);

	const uint32 CurrentPid = FPlatformProcess::GetCurrentProcessId();

	// NOTE: FPlatformProcess::ExecutablePath() is stale if we were moved while running.
	const FString OriginalThisExePath = FPlatformProcess::GetApplicationName(CurrentPid);
	const FString ThisExeDir = FPaths::GetPath(OriginalThisExePath);
	const FString ThisExeFilename = FPaths::GetCleanFilename(OriginalThisExePath);
	
	const FString MovedListenerPath = FPaths::EngineIntermediateDir() / TEXT("Switchboard") / TEXT("old_") + ThisExeFilename;

	//
	// Weird hackery to get around Windows locking down an executable file on disk whilst running.
	// We move the (locked) file (which is allowed), and then make a copy (which is no longer linked to the running process)

	if (!IFileManager::Get().Move(*MovedListenerPath, *OriginalThisExePath, true))
	{
		const uint32 LastError = FPlatformMisc::GetLastError();
		const FString ErrorMsg = FString::Printf(TEXT("Unable to move listener exe to \"%s\" (error code %u)"), *MovedListenerPath, LastError);
		UE_LOGF(LogSwitchboard, Error, "Free listener binary: %ls", *ErrorMsg);
		SendMessage(CreateTaskDeclinedMessage(InFreeListenerBinaryTask, ErrorMsg, {}), InFreeListenerBinaryTask.Recipient);
		return false;
	}

	if (IFileManager::Get().Copy(*OriginalThisExePath, *MovedListenerPath, true, true) != ECopyResult::COPY_OK)
	{
		const uint32 LastError = FPlatformMisc::GetLastError();
		const FString ErrorMsg = FString::Printf(TEXT("Unable to copy listener exe back to \"%s\" (error code %u)"), *OriginalThisExePath, LastError);
		UE_LOGF(LogSwitchboard, Error, "Free listener binary: %ls", *ErrorMsg);
		SendMessage(CreateTaskDeclinedMessage(InFreeListenerBinaryTask, ErrorMsg, {}), InFreeListenerBinaryTask.Recipient);
		return false;
	}

	return true;
}

bool FSwitchboardListener::Task_StartLiveLogStream(const FSwitchboardStartLiveLogStreamTask& InStartLiveLogStreamTask)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::Task_StartLiveLogStream);

	const FString& LogFilePath = InStartLiveLogStreamTask.LogFilePath;
	const FIPv4Endpoint& ClientEndpoint = InStartLiveLogStreamTask.Recipient;

	// Check if the given log file exists
	if (!FPaths::FileExists(LogFilePath))
	{
		const FString ErrorMsg = FString::Printf(TEXT("Log file does not exist: %s"), *LogFilePath);

		UE_LOGF(LogSwitchboard, Error, "Start live log stream: %ls", *ErrorMsg);

		SendMessage(CreateTaskDeclinedMessage(InStartLiveLogStreamTask, ErrorMsg, {}), ClientEndpoint);

		return false;
	}

	// Check if stream already exists for this file and client
	{
		UE::TUniqueLock<UE::FMutex> Lock(LiveLogStreamsMutex);
		for (const FLiveLogStream& Stream : ActiveLogStreams)
		{
			if (Stream.LogFilePath == LogFilePath && Stream.ClientEndpoint == ClientEndpoint)
			{
				const FString ErrorMsg = TEXT("Live log stream already active for this file and client");

				SendMessage(CreateTaskDeclinedMessage(InStartLiveLogStreamTask, ErrorMsg, {}), ClientEndpoint);
				return false;
			}
		}

		// Add new stream
		FLiveLogStream NewStream(LogFilePath, ClientEndpoint, InStartLiveLogStreamTask.bTailOnly, InStartLiveLogStreamTask.ProcessUUID, InStartLiveLogStreamTask.StartOffset);

		if (InStartLiveLogStreamTask.ProcessUUID.IsValid())
		{
			UE_LOGF(LogSwitchboard, Log, "Starting live log stream for %ls with associated process UUID: %ls", *LogFilePath, *InStartLiveLogStreamTask.ProcessUUID.ToString());
		}
		else
		{
			UE_LOGF(LogSwitchboard, Log, "Starting live log stream for %ls with no associated process UUID", *LogFilePath);
		}

		// Handle start position based on StartOffset parameter
		if (InStartLiveLogStreamTask.StartOffset >= 0)
		{
			// Use explicit start offset if provided
			NewStream.LastFilePosition = InStartLiveLogStreamTask.StartOffset;
			UE_LOGF(LogSwitchboard, Log, "Starting live log stream from byte offset: %lld", NewStream.LastFilePosition);
		}
		else if (!NewStream.bTailOnly)
		{
			// Start from the beginning
			NewStream.LastFilePosition = 0;
		}
		else
		{
			// Start from end of file for tail-only mode
			TUniquePtr<IFileHandle> FileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenRead(*LogFilePath));
			if (FileHandle)
			{
				NewStream.LastFilePosition = FileHandle->Size();
			}
		}

		ActiveLogStreams.Add(MoveTemp(NewStream));
	}

	// Send success response
	SendMessage(CreateCommandAcceptedMessage(InStartLiveLogStreamTask.TaskID), ClientEndpoint);
	UE_LOGF(LogSwitchboard, Log, "Started live log stream for %ls", *LogFilePath);

	return true;
}

bool FSwitchboardListener::Task_StopLiveLogStream(const FSwitchboardStopLiveLogStreamTask& InStopLiveLogStreamTask)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::Task_StopLiveLogStream);

	const FString& LogFilePath = InStopLiveLogStreamTask.LogFilePath;
	const FIPv4Endpoint& ClientEndpoint = InStopLiveLogStreamTask.Recipient;

	bool bStreamFound = false;
	{
		UE::TUniqueLock<UE::FMutex> Lock(LiveLogStreamsMutex);
		for (int32 StreamIdx = ActiveLogStreams.Num() - 1; StreamIdx >= 0; --StreamIdx)
		{
			const FLiveLogStream& Stream = ActiveLogStreams[StreamIdx];
			if (Stream.LogFilePath == LogFilePath && Stream.ClientEndpoint == ClientEndpoint)
			{
				ActiveLogStreams.RemoveAt(StreamIdx);
				bStreamFound = true;
				break;
			}
		}
	}

	if (bStreamFound)
	{
		SendMessage(CreateCommandAcceptedMessage(InStopLiveLogStreamTask.TaskID), ClientEndpoint);
		UE_LOGF(LogSwitchboard, Log, "Stopped live log stream for %ls", *LogFilePath);

		return true;
	}
	else
	{
		const FString ErrorMsg = TEXT("No active live log stream found for this file and client");
		SendMessage(CreateTaskDeclinedMessage(InStopLiveLogStreamTask, ErrorMsg, {}), ClientEndpoint);

		return false;
	}
}

void FSwitchboardListener::HandleLiveLogStreams()
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::HandleLiveLogStreams);

	const double CurrentTime = FPlatformTime::Seconds();
	constexpr double StreamUpdateInterval = 0.1; // 100 ms between updates

	UE::TUniqueLock<UE::FMutex> Lock(LiveLogStreamsMutex);

	// Collect indices of streams whose associated processes have ended. 
	// If they have, then the stream are removed, effectively stopping them.

	TArray<int32> StreamsToRemove;

	for (int32 StreamIdx = 0; StreamIdx < ActiveLogStreams.Num(); ++StreamIdx)
	{
		FLiveLogStream& Stream = ActiveLogStreams[StreamIdx];

		// Check if the associated process is still running (if we have a valid UUID)

		if (Stream.AssociatedProcessUUID.IsValid())
		{
			bool bProcessStillRunning = false;

			for (const TSharedPtr<FRunningProcess>& Process : RunningProcesses)
			{
				if (Process && Process->UUID == Stream.AssociatedProcessUUID)
				{
					bProcessStillRunning = true;
					break;
				}
			}

			if (!bProcessStillRunning)
			{
				UE_LOGF(LogSwitchboard, Log, "Stopping live log stream for %ls because the associated Unreal process has ended", *Stream.LogFilePath);
				StreamsToRemove.Add(StreamIdx);
				continue;
			}
		}
	}

	// Remove streams whose processes have ended (in reverse order to preserve indices)

	for (int32 StreamIdx = StreamsToRemove.Num() - 1; StreamIdx >= 0; --StreamIdx)
	{
		const FLiveLogStream& StreamToRemove = ActiveLogStreams[StreamsToRemove[StreamIdx]];

		// Notify Switchboard that the stream has terminated
		const FString TerminationMessage = CreateLiveLogStreamTerminatedMessage(StreamToRemove.LogFilePath, TEXT("process terminated"));
		SendMessage(TerminationMessage, StreamToRemove.ClientEndpoint);

		ActiveLogStreams.RemoveAt(StreamsToRemove[StreamIdx]);
	}

	// Process remaining active streams
	for (FLiveLogStream& Stream : ActiveLogStreams)
	{
		// Rate limiting: only update every StreamUpdateInterval seconds per stream
		if (CurrentTime - Stream.LastUpdateTime < StreamUpdateInterval)
		{
			continue;
		}

		Stream.LastUpdateTime = CurrentTime;

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		if (!PlatformFile.FileExists(*Stream.LogFilePath))
		{
			UE_LOGF(LogSwitchboard, Warning, "Live log file does not exist: %ls", *Stream.LogFilePath);
			continue;
		}

		// Use shared read mode to allow reading while Unreal is writing to the file (otherwise open will fail)

		TUniquePtr<IFileHandle> FileHandle(PlatformFile.OpenRead(*Stream.LogFilePath, true));

		if (!FileHandle)
		{
			UE_LOGF(LogSwitchboard, Warning, "Live log file cannot be opened for reading: %ls", *Stream.LogFilePath);
			continue;
		}

		const int64 CurrentFileSize = FileHandle->Size();
		if (CurrentFileSize <= Stream.LastFilePosition)
		{
			// Nothing to send.
			continue;
		}

		// Calculate how much new content to read (limit to 32KB per update)

		constexpr int64 MaxReadSize = 32768;
		const int64 AvailableBytes = CurrentFileSize - Stream.LastFilePosition;
		const int64 ReadSize = FMath::Min(AvailableBytes, MaxReadSize);

		TArray<uint8> Buffer;
		Buffer.SetNumUninitialized(ReadSize);

		if (FileHandle->Seek(Stream.LastFilePosition) && FileHandle->Read(Buffer.GetData(), ReadSize))
		{
			Stream.LastFilePosition += ReadSize;

			TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);
			JsonObject->SetStringField(TEXT("command"), TEXT("live log data"));
			JsonObject->SetStringField(TEXT("log_file_path"), Stream.LogFilePath);
			JsonObject->SetNumberField(TEXT("file_position"), Stream.LastFilePosition - ReadSize);
			JsonObject->SetNumberField(TEXT("content_size"), ReadSize);

			// Compress if enabled and chunk is large enough to benefit from compression

			const int64 CompressionThreshold = 1024; // 1KB threshold

			if (ReadSize >= CompressionThreshold)
			{
				// Try to compress the data
				const int32 CompressedUpperBound = FCompression::CompressMemoryBound(NAME_Zlib, ReadSize);
				TArray<uint8> CompressedData;
				CompressedData.SetNumUninitialized(CompressedUpperBound);

				int32 CompressedSize = CompressedUpperBound;
				if (FCompression::CompressMemory(NAME_Zlib, CompressedData.GetData(), CompressedSize, Buffer.GetData(), ReadSize, COMPRESS_BiasSpeed))
				{
					// Only use compression if it actually saves space
					if (CompressedSize < ReadSize)
					{
						CompressedData.SetNum(CompressedSize);
						FString CompressedBase64Content = FBase64::Encode(CompressedData.GetData(), CompressedSize);
						JsonObject->SetStringField(TEXT("content_compressed_base64"), CompressedBase64Content);
						JsonObject->SetNumberField(TEXT("uncompressed_size"), ReadSize);
					}
					else
					{
						// Compression didn't help, use uncompressed
						FString Base64Content = FBase64::Encode(Buffer.GetData(), ReadSize);
						JsonObject->SetStringField(TEXT("content_base64"), Base64Content);
					}
				}
				else
				{
					// Compression failed, use uncompressed
					FString Base64Content = FBase64::Encode(Buffer.GetData(), ReadSize);
					JsonObject->SetStringField(TEXT("content_base64"), Base64Content);
				}
			}
			else
			{
				// No compression (either disabled or chunk too small)
				FString Base64Content = FBase64::Encode(Buffer.GetData(), ReadSize);
				JsonObject->SetStringField(TEXT("content_base64"), Base64Content);
			}

			FString OutputString;
			TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
			FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer);

			SendMessage(OutputString, Stream.ClientEndpoint);
		}
	}
}

void FSwitchboardListener::CleanUpDisconnectedSockets()
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::CleanUpDisconnectedSockets);

	UE::TUniqueLock<UE::FRecursiveMutex> ConnectionsScopeLock(ConnectionsLock);

	const double CurrentTime = FPlatformTime::Seconds();
	for (const TPair<FIPv4Endpoint, FConnectionRef>& Connection : ConnectionsByEndpoint)
	{
		const FIPv4Endpoint& Client = Connection.Key;
		const float ClientTimeout = Connection.Value->InactiveTimeout;
		const double InactiveTime = CurrentTime - Connection.Value->LastActivityTime;
		if (ClientTimeout > 0.0f && InactiveTime > ClientTimeout)
		{
			UE_LOGF(LogSwitchboard, Warning, "Client %ls has been inactive for more than %.1fs -- closing connection", *Client.ToString(), ClientTimeout);
			TUniquePtr<FSwitchboardDisconnectTask> DisconnectTask = MakeUnique<FSwitchboardDisconnectTask>(FGuid(), Client);
			DisconnectTasks.Enqueue(MoveTemp(DisconnectTask));
		}
	}

	while (!DisconnectTasks.IsEmpty())
	{
		TUniquePtr<FSwitchboardTask> Task;
		DisconnectTasks.Dequeue(Task);
		const FSwitchboardDisconnectTask& DisconnectTask = static_cast<const FSwitchboardDisconnectTask&>(*Task);
		const FIPv4Endpoint& Client = DisconnectTask.Recipient;

		if (RedeployStatus.InProgress() && Client == RedeployStatus.RequestingClient)
		{
			UE_LOGF(LogSwitchboard, Warning, "Client %ls disconnected before redeploy completed", *Client.ToString());
			RollbackRedeploy();
		}

		UE_LOGF(LogSwitchboard, Display, "Client %ls disconnecting", *Client.ToString());

		if (FConnectionRef* MaybeConnection = ConnectionsByEndpoint.Find(Client))
		{
			if ((*MaybeConnection)->QuicConn)
			{
				QuicApi->ConnectionShutdown((*MaybeConnection)->QuicConn, QUIC_CONNECTION_SHUTDOWN_FLAG_SILENT, 0);
			}
		}
	}
}

void FSwitchboardListener::HandleStdout(const TSharedPtr<FRunningProcess>& Process)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::HandleStdout);

	TArray<uint8> Stdout, Stderr;
	if (FPlatformProcess::ReadPipeToArray(Process->StdoutParentReadPipe, Stdout))
	{
		Process->Stdout.Append(Stdout);
	}

	if (!Process->bCombinedStdoutStderr)
	{
		if (FPlatformProcess::ReadPipeToArray(Process->StderrParentReadPipe, Stderr))
		{
			Process->Stderr.Append(Stderr);
		}
	}

	// If there was new output, update the clients
	if (Process->bUpdateClientsWithStdout && (Stdout.Num() || Stderr.Num()))
	{
		FSwitchboardProgramStdout Packet;

		Packet.Process.Uuid = Process->UUID.ToString();
		Packet.Process.Name = Process->Name;
		Packet.Process.Path = Process->Path;
		Packet.Process.Caller = Process->Caller;
		Packet.Process.Pid = Process->PID;

		if (Stdout.Num())
		{
			Packet.PartialStdoutB64 = FBase64::Encode(Stdout);
		}

		if (Stderr.Num())
		{
			Packet.PartialStderrB64 = FBase64::Encode(Stderr);
		}

		UE::TUniqueLock<UE::FRecursiveMutex> ConnectionsScopeLock(ConnectionsLock);
		for (const TPair<FIPv4Endpoint, FConnectionRef>& Connection : ConnectionsByEndpoint)
		{
			const FIPv4Endpoint& ClientEndpoint = Connection.Key;
			SendMessage(CreateMessage(Packet), ClientEndpoint);
		}
	}
}

void FSwitchboardListener::HandleRunningProcesses(TArray<TSharedPtr<FRunningProcess>>& Processes, bool bNotifyThatProgramEnded)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::HandleRunningProcesses);

	// Reads pipe and cleans up dead processes from the array.
	for (auto Iter = Processes.CreateIterator(); Iter; ++Iter)
	{
		TSharedPtr<FRunningProcess> Process = *Iter;

		check(Process.IsValid());

		if (Process->bPendingKill)
		{
			continue;
		}

		if (Process->Handle.IsValid())
		{
			HandleStdout(Process);

			if (!FPlatformProcess::IsProcRunning(Process->Handle))
			{
				// A final read can be necessary to avoid truncating the output.
				HandleStdout(Process);

				int32 ReturnCode = 0;
				FPlatformProcess::GetProcReturnCode(Process->Handle, &ReturnCode);
				UE_LOGF(LogSwitchboard, Display, "Process %d (%ls) exited with returncode: %d",
					Process->PID, *Process->Name, ReturnCode);

				if (ReturnCode != 0)
				{
					if (Process->Stdout.Num() > 0)
					{
						FUTF8ToTCHAR StdoutConv(reinterpret_cast<const ANSICHAR*>(Process->Stdout.GetData()), Process->Stdout.Num());
						UE_LOGF(LogSwitchboard, Display, "Stdout:\n%.*ls", StdoutConv.Length(), StdoutConv.Get());
					}

					if (Process->Stderr.Num() > 0)
					{
						FUTF8ToTCHAR StderrConv(reinterpret_cast<const ANSICHAR*>(Process->Stderr.GetData()), Process->Stderr.Num());
						UE_LOGF(LogSwitchboard, Display, "Stderr:\n%.*ls", StderrConv.Length(), StderrConv.Get());
					}
				}

				// Let the clock manager know that the Pid ended.
				if (GpuClockerManager.IsValid())
				{
					GpuClockerManager->PidEnded(Process->PID);
				}

				// Notify remote client, which implies that this is a program managed by it.
				if (bNotifyThatProgramEnded)
				{
					FSwitchboardProgramEnded Packet;

					Packet.Process.Uuid = Process->UUID.ToString();
					Packet.Process.Name = Process->Name;
					Packet.Process.Path = Process->Path;
					Packet.Process.Caller = Process->Caller;
					Packet.Process.Pid = Process->PID;
					Packet.Returncode = ReturnCode;
					Packet.StdoutB64 = FBase64::Encode(Process->Stdout);
					Packet.StderrB64 = FBase64::Encode(Process->Stderr);

					{
						UE::TUniqueLock<UE::FRecursiveMutex> ConnectionsScopeLock(ConnectionsLock);
						for (const TPair<FIPv4Endpoint, FConnectionRef>& Connection : ConnectionsByEndpoint)
						{
							const FIPv4Endpoint& ClientEndpoint = Connection.Key;
							SendMessage(CreateMessage(Packet), ClientEndpoint);
						}
					}

					// Kill its monitor to avoid potential zombies (unless it is already pending kill)
					{
						TSharedPtr<FRunningProcess>* FlipModeMonitorPtr = FlipModeMonitors.FindByPredicate([&](const TSharedPtr<FRunningProcess>& FlipMonitor)
						{
							check(FlipMonitor.IsValid());
							return !FlipMonitor->bPendingKill && (FlipMonitor->UUID == Process->UUID);
						});

						if (FlipModeMonitorPtr)
						{
							check((*FlipModeMonitorPtr).IsValid());

							FSwitchboardKillTask Task(FGuid(), (*FlipModeMonitorPtr)->Recipient, (*FlipModeMonitorPtr)->UUID);
							Task_KillProcess(Task);
						}
					}
				}

				Process->OnProcessEnded.Broadcast();

				FPlatformProcess::CloseProc(Process->Handle);
				Process->ClosePipes();

				Iter.RemoveCurrent();
			}
		}
	}
}

bool FSwitchboardListener::SendMessage(const FString& InMessage, const FIPv4Endpoint& InEndpoint)
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::SendMessage);

	if (InEndpoint == InvalidEndpoint)
	{
		return false;
	}

	UE::TUniqueLock<UE::FRecursiveMutex> ConnectionsScopeLock(ConnectionsLock);

	if (!ConnectionsByEndpoint.Contains(InEndpoint))
	{
		// this happens when a client disconnects while a task it had issued is not finished
		UE_LOGF(LogSwitchboard, Verbose, "Trying to send message to disconnected client %ls", *InEndpoint.ToString());
		return false;
	}

	FConnectionRef Connection = ConnectionsByEndpoint[InEndpoint];

	UE_LOGF(LogSwitchboardProtocol, Verbose, "Sending message to client '%ls' : %ls", *InEndpoint.ToString(), *InMessage);

	uint64 Utf8Length = FPlatformString::ConvertedLength<UTF8CHAR>(*InMessage, InMessage.Len() + 1);
	FByteArrayRef SendArray = MakeShared<TArray<uint8>>();
	SendArray->SetNumUninitialized(Utf8Length);
	FPlatformString::Convert((UTF8CHAR*)SendArray->GetData(), SendArray->Num(), *InMessage, InMessage.Len() + 1);

	UE::TUniqueLock<UE::FMutex> SendLock(Connection->SendLock);
	TSharedPtr<FQuicBuffer> SendBuffer = MakeShared<FQuicBuffer>(MoveTemp(SendArray));
	Connection->SendBuffers.Enqueue(SendBuffer);
	QuicApi->StreamSend(Connection->QuicStream, &SendBuffer->QuicBuffer, 1, QUIC_SEND_FLAG_NONE, SendBuffer.Get());

	return true;
}

void FSwitchboardListener::SendMessageFutures()
{
	SWITCHBOARD_TRACE_CPUPROFILER_EVENT_SCOPE(FSwitchboardListener::SendMessageFutures);

	for (auto Iter = MessagesFutures.CreateIterator(); Iter; ++Iter)
	{
		FSwitchboardMessageFuture& MessageFuture = *Iter;

		if (!MessageFuture.Future.IsReady())
		{
			continue;
		}

		FString Message = MessageFuture.Future.Get();
		if (!Message.IsEmpty())
		{
			SendMessage(Message, MessageFuture.InEndpoint);
		}

		Iter.RemoveCurrent();
	}
}
