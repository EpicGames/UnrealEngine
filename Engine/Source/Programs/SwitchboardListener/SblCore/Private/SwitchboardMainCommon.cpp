// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardListener.h"
#include "SwitchboardListenerApp.h"
#include "SwitchboardListenerVersion.h"

#include "Async/TaskGraphInterfaces.h"
#include "HAL/ExceptionHandling.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/ThreadingBase.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Interfaces/IPluginManager.h"
#include "LaunchEngineLoop.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ScopeExit.h"
#include "Modules/BuildVersion.h"
#include "Modules/ModuleManager.h"
#include "MsQuicRuntimeModule.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "UObject/UObjectBase.h"


DEFINE_LOG_CATEGORY(LogSwitchboard);


#if !defined(SWITCHBOARD_LISTENER_EXCLUDE_MAIN)

int32 InitEngine()
{
	const TCHAR* CommandLine = FCommandLine::Get();
	const int32 InitResult = GEngineLoop.PreInit(CommandLine);
	if (InitResult != 0)
	{
		return InitResult;
	}

	if (GLog)
	{
		GLog->EnableBacklog(true);
	}

	ProcessNewlyLoadedUObjects();
	FModuleManager::Get().StartProcessingNewlyLoadedObjects();

	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PreDefault);
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::Default);

	return 0;
}


bool InitSocketSystem()
{
	EModuleLoadResult LoadResult;
	FModuleManager::Get().LoadModuleWithFailureReason(TEXT("Sockets"), LoadResult);

	FIPv4Endpoint::Initialize();

	return LoadResult == EModuleLoadResult::Success;
}


// There's a subtle potential race condition with the parent PID being recycled
// after the parent listener has actually exited. We open a handle first, then
// ensure we have the right process by enumerating processes to check that it's
// actually our parent process. At that point, the open handle is unambiguous.
FProcHandle CheckRedeploy(const FSwitchboardCommandLineOptions& Options)
{
	if (!Options.RedeployFromPid.IsSet())
	{
		return FProcHandle();
	}

	const uint32 ExpectedParentPid = Options.RedeployFromPid.GetValue();
	const FProcHandle ExpectedParentProc = FPlatformProcess::OpenProcess(ExpectedParentPid);
	if (!ExpectedParentProc.IsValid())
	{
		UE_LOGF(LogSwitchboard, Error, "Couldn't get handle to redeploy parent process; you'll need to remove the old executable manually");
		return FProcHandle();
	}

	const uint32 CurrentPid = FPlatformProcess::GetCurrentProcessId();
	FPlatformProcess::FProcEnumerator ProcessEnumerator;
	while (ProcessEnumerator.MoveNext())
	{
		FPlatformProcess::FProcEnumInfo CurrentProcInfo = ProcessEnumerator.GetCurrent();
		const bool bIsCurrentProc = CurrentProcInfo.GetPID() == CurrentPid;
		const bool bHasExpectedParent = CurrentProcInfo.GetParentPID() == ExpectedParentPid;
		if (bIsCurrentProc)
		{
			if (bHasExpectedParent)
			{
				return ExpectedParentProc;
			}
			else
			{
				UE_LOGF(LogSwitchboard, Error, "Redeploy expected parent PID of %u, but found %u; you'll need to remove the old executable manually",
					ExpectedParentPid, CurrentProcInfo.GetParentPID());

				return FProcHandle();
			}
		}
	}

	// Shouldn't be possible.
	check(false);
	UE_LOGF(LogSwitchboard, Error, "Failed to enumerate our own PID!");

	return FProcHandle();
}


// Wait for the old listener parent process to exit and delete its executable.
bool HandleRedeploy(FProcHandle& RedeployParentProc, uint32 RedeployParentPid)
{
	const FString& OldLauncherPath = FPlatformProcess::GetApplicationName(RedeployParentPid);

	// Get handle to existing IPC semaphore.
	const FString IpcSemaphoreName = FSwitchboardListener::GetIpcSemaphoreName(RedeployParentPid);
	FPlatformProcess::FSemaphore* IpcSemaphore = FPlatformProcess::NewInterprocessSynchObject(IpcSemaphoreName, false);
	ON_SCOPE_EXIT
	{
		if (IpcSemaphore)
		{
			FPlatformProcess::DeleteInterprocessSynchObject(IpcSemaphore);
		}
	};

	if (IpcSemaphore == nullptr)
	{
		UE_LOGF(LogSwitchboard, Fatal, "Error opening redeploy IPC semaphore");
		return false;
	}

	// This release signals the old parent launcher that we're initialized and it can shut down.
	IpcSemaphore->Unlock();

	if (FPlatformProcess::IsProcRunning(RedeployParentProc))
	{
		UE_LOGF(LogSwitchboard, Display, "Waiting for previous listener to shut down...");
		int8 SecondsToWait = 5;
		bool bStillRunning = true;
		do
		{
			FPlatformProcess::Sleep(1.0f);
			--SecondsToWait;
			bStillRunning = FPlatformProcess::IsProcRunning(RedeployParentProc);
		} while (bStillRunning && SecondsToWait > 0);

		if (bStillRunning)
		{
			UE_LOGF(LogSwitchboard, Warning, "Forcibly terminating previous listener");
			if (RedeployParentProc.IsValid())
			{
				FPlatformProcess::TerminateProc(RedeployParentProc);
				FPlatformProcess::CloseProc(RedeployParentProc);
			}
			bStillRunning = FPlatformProcess::IsApplicationRunning(RedeployParentPid);
		}

		if (bStillRunning)
		{
			UE_LOGF(LogSwitchboard, Error,
				"Unable to shut down previous listener at %ls; you'll need to remove the old executable manually",
				*OldLauncherPath);
			return false;
		}
	}

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.DeleteFile(*OldLauncherPath))
	{
		UE_LOGF(LogSwitchboard, Error,
			"Unable to remove previous listener at %ls; you may need to remove the old executable manually",
			*OldLauncherPath);
		return false;
	}

	return true;
}


bool SwitchboardListenerMainInit(const FSwitchboardCommandLineOptions& InOptions)
{
	const int32 InitResult = InitEngine();

	UE_LOGF(LogSwitchboard, Display, "SwitchboardListener %u.%u.%u", SBLISTENER_VERSION_MAJOR, SBLISTENER_VERSION_MINOR, SBLISTENER_VERSION_PATCH);

	if (InitResult != 0)
	{
		UE_LOGF(LogSwitchboard, Fatal, "Could not initialize engine, Error code: %d", InitResult);
		RequestEngineExit(TEXT("Engine init failure"));
		return false;
	}

	UE_LOGF(LogSwitchboard, Display, "Successfully initialized engine.");

	if (!InitSocketSystem())
	{
		UE_LOGF(LogSwitchboard, Fatal, "Could not initialize socket system!");
		RequestEngineExit(TEXT("Socket init failure"));
		return false;
	}

	UE_LOGF(LogSwitchboard, Display, "Successfully initialized socket system.");

	if (!FMsQuicRuntimeModule::InitRuntime())
	{
		UE_LOGF(LogSwitchboard, Fatal, "Could not initialize MsQuic runtime.");
		RequestEngineExit(TEXT("MsQuic init failure"));
		return false;
	}

	FProcHandle RedeployParentProc = CheckRedeploy(InOptions);
	if (RedeployParentProc.IsValid())
	{
		UE_LOGF(LogSwitchboard, Display, "Performing redeploy");
		HandleRedeploy(RedeployParentProc, InOptions.RedeployFromPid.GetValue());
	}

	return true;
}


bool SwitchboardListenerMainShutdown()
{
	// Cleanup. See also: FEngineLoop::AppPreExit and FEngineLoop::AppExit
	UE_LOGF(LogExit, Log, "Preparing to exit.");

	FCoreDelegates::OnPreExit.Broadcast();
	FCoreDelegates::OnExit.Broadcast();

	FModuleManager::Get().UnloadModulesAtShutdown();

	if (GThreadPool != nullptr)
	{
		GThreadPool->Destroy();
	}
	if (GBackgroundPriorityThreadPool != nullptr)
	{
		GBackgroundPriorityThreadPool->Destroy();
	}
	if (GIOThreadPool != nullptr)
	{
		GIOThreadPool->Destroy();
	}

#if STATS
	FThreadStats::StopThread();
#endif

	FTaskGraphInterface::Shutdown();

#if WITH_APPLICATION_CORE
	FPlatformApplicationMisc::TearDown();
#endif
	FPlatformMisc::PlatformTearDown();

	if (GConfig)
	{
		GConfig->Exit();
		delete GConfig;
		GConfig = nullptr;
	}

	if (GLog)
	{
		GLog->TearDown();
	}

	FTextLocalizationManager::TearDown();
	FInternationalization::TearDown();

	FTraceAuxiliary::Shutdown();

	return true;
}


extern int32 SwitchboardListenerMain();


int32 SwitchboardListenerMainWrapper()
{
	int32 ExitCode;
	if (FPlatformMisc::IsDebuggerPresent())
	{
		ExitCode = SwitchboardListenerMain();
	}
	else
	{
#if PLATFORM_WINDOWS && !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__try
#endif
		{
			// SetCrashHandler(nullptr) sets up default behavior for Linux and Mac interfacing with CrashReportClient
			FPlatformMisc::SetCrashHandler(nullptr);
			GIsGuarded = true;
			ExitCode = SwitchboardListenerMain();
			GIsGuarded = false;
		}
#if PLATFORM_WINDOWS && !PLATFORM_SEH_EXCEPTIONS_DISABLED
		__except (ReportCrash(GetExceptionInformation()))
		{
			if (GError)
			{
				GError->HandleError();
			}

			// RequestExit(bForce==True) terminates this process with a specific exit code and does not return.
			FPlatformMisc::RequestExit(true);
			ExitCode = 1; // Suppress compiler warning.
		}
#endif
	}

	return ExitCode;
}

#endif // #if !defined(SWITCHBOARD_LISTENER_EXCLUDE_MAIN)
