// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/UnrealString.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "SwitchboardTasks.h"

struct FSwitchboardMessageFuture
{
	/** Endpoint where this message should be sent to */
	FIPv4Endpoint InEndpoint;

	/** Equivalence hash of the original task */
	uint32 EquivalenceHash;

	/** Future containing message contents */
	TFuture<FString> Future;
};