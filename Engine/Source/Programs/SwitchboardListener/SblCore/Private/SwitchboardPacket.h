// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SwitchboardListenerVersion.h"

#include "SwitchboardPacket.generated.h"


USTRUCT()
struct FSwitchboardPacket
{
	GENERATED_BODY()

	UPROPERTY()
	FString Command;

	UPROPERTY()
	bool bAck;

	UPROPERTY()
	FString Error;
};


USTRUCT()
struct FSwitchboardStateRunningProcess
{
	GENERATED_BODY()

	UPROPERTY()
	FString Uuid;

	UPROPERTY()
	FString Path;

	UPROPERTY()
	FString Name;

	UPROPERTY()
	FString Caller;

	UPROPERTY()
	uint32 Pid;
};

// This is an initial snapshot, sent upon connecting.
USTRUCT()
struct FSwitchboardStatePacket : public FSwitchboardPacket
{
	GENERATED_BODY()

	FSwitchboardStatePacket() : FSwitchboardPacket()
	{
		Command = TEXT("state");
		bAck = true;

		const uint32 Major = SBLISTENER_VERSION_MAJOR;
		const uint32 Minor = SBLISTENER_VERSION_MINOR;
		const uint32 Patch = SBLISTENER_VERSION_PATCH;

		Version = (Major << 16) | (Minor << 8) | (Patch);
	}

	/** All active child processes launched via this Listener. */
	UPROPERTY()
	TArray<FSwitchboardStateRunningProcess> RunningProcesses;

	/** Reports the SwitchboardListener version back to the client. */
	UPROPERTY()
	uint32 Version;

	/** Whether simultaneous multi-threading (aka Hyper-Threading) is enabled. */
	UPROPERTY()
	bool bProcessorSMT;

	/** Operating system version "label". See FPlatformMisc::GetOSVersions. */
	UPROPERTY()
	FString OsVersionLabel;

	/** Operating system version "sub-label". See FPlatformMisc::GetOSVersions. */
	UPROPERTY()
	FString OsVersionLabelSub;

	/** Operating system version, returned by FPlatformMisc::GetOSVersion. */
	UPROPERTY()
	FString OsVersionNumber;

	/** Total physical memory (used + available) in bytes. */
	UPROPERTY()
	uint64 TotalPhysicalMemory;

	/** The name of the platform as it appears in Engine/Binaries */
	UPROPERTY()
	FString PlatformBinaryDirectory;
};

USTRUCT()
struct FSwitchboardProgramStdout : public FSwitchboardPacket
{
	GENERATED_BODY()

	FSwitchboardProgramStdout() : FSwitchboardPacket()
	{
		Command = TEXT("programstdout");
		bAck = true;
	}

	UPROPERTY()
	FSwitchboardStateRunningProcess Process;

	UPROPERTY()
	FString PartialStdoutB64;

	UPROPERTY()
	FString PartialStderrB64;
};

USTRUCT()
struct FSwitchboardProgramEnded : public FSwitchboardPacket
{
	GENERATED_BODY()

	FSwitchboardProgramEnded() : FSwitchboardPacket()
	{
		Command = TEXT("program ended");
		bAck = true;
	}

	UPROPERTY()
	FSwitchboardStateRunningProcess Process;

	UPROPERTY()
	int32 Returncode;

	UPROPERTY()
	FString StdoutB64;

	UPROPERTY()
	FString StderrB64;
};

USTRUCT()
struct FSwitchboardProgramStarted : public FSwitchboardPacket
{
	GENERATED_BODY()

	FSwitchboardProgramStarted() : FSwitchboardPacket()
	{
		Command = TEXT("program started");
		bAck = true;
	}

	UPROPERTY()
	FSwitchboardStateRunningProcess Process;
};

USTRUCT()
struct FSwitchboardProgramKilled : public FSwitchboardPacket
{
	GENERATED_BODY()

	FSwitchboardProgramKilled() : FSwitchboardPacket()
	{
		Command = TEXT("program killed");
		bAck = true;
	}

	UPROPERTY()
	FSwitchboardStateRunningProcess Process;
};
