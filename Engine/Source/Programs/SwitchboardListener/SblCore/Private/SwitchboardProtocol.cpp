// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardProtocol.h"

#include "SwitchboardPacket.h"
#include "SwitchboardTasks.h"
#include "SyncStatus.h"

#include "Dom/JsonValue.h"
#include "JsonObjectConverter.h"
#include "Logging/StructuredLog.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"


DEFINE_LOG_CATEGORY(LogSwitchboardProtocol);


FString CreateMessage(const TMap<FString, FString>& InFields)
{
	FString Message;
	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Message);
	JsonWriter->WriteObjectStart();
	for (const TPair<FString, FString>& Field : InFields)
	{
		JsonWriter->WriteValue(Field.Key, Field.Value);
	}
	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();
	return Message;
}


FString CreateMessage(const FString& InStateDescription, bool bInState, const TMap<FString, FString>& InAdditionalFields)
{
	FString Message;
	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Message);
	JsonWriter->WriteObjectStart();
	JsonWriter->WriteValue(InStateDescription, bInState); // TODO: Phase out this field and replace with two below because parser now needs to check all possibilities.
	JsonWriter->WriteValue(TEXT("command"), InStateDescription);
	JsonWriter->WriteValue(TEXT("bAck"), bInState);
	for (const auto& Value : InAdditionalFields)
	{
		JsonWriter->WriteValue(Value.Key, Value.Value);
	}
	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();
	return Message;
}

FString CreateTaskDeclinedMessage(const FSwitchboardTask& InTask, const FString& InErrorMessage, const TMap<FString, FString>& InAdditionalFields)
{
	TMap<FString, FString> AdditionalFields = InAdditionalFields;

	AdditionalFields.Add(TEXT("id"), InTask.TaskID.ToString());
	AdditionalFields.Add(TEXT("error"), InErrorMessage);

	return CreateMessage(InTask.GetCommandName(), false, AdditionalFields);
}

FString CreateCommandAcceptedMessage(const FGuid& InMessageID)
{
	return CreateMessage(TEXT("command accepted"), true, { { TEXT("id"), InMessageID.ToString() } });
}

FString CreateCommandDeclinedMessage(const FGuid& InMessageID, const FString& InErrorMessage)
{
	return CreateMessage(TEXT("command accepted"), false, { { TEXT("id"), InMessageID.ToString() }, {TEXT("error"), InErrorMessage} });
}

FString CreateSyncStatusMessage(const FSyncStatus& SyncStatus, ESyncStatusRequestFlags RequestFlags)
{
	FString SyncStatusJsonString;
	const bool bJsonStringOk = FJsonObjectConverter::UStructToJsonObjectString(SyncStatus, SyncStatusJsonString);

	check(bJsonStringOk);

	FString Message;

	using RequestFlagsIntType = std::underlying_type_t<ESyncStatusRequestFlags>;
	RequestFlagsIntType RequestFlagsValue = static_cast<RequestFlagsIntType>(RequestFlags);

	TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&Message);
	JsonWriter->WriteObjectStart();
	JsonWriter->WriteValue(TEXT("get sync status"), true); // TODO: Phase out this field and replace with two below because parser now needs to check all possibilities.
	JsonWriter->WriteValue(TEXT("command"), TEXT("get sync status"));
	JsonWriter->WriteValue(TEXT("bAck"), true);
	JsonWriter->WriteValue(TEXT("request_flags"), RequestFlagsValue);
	JsonWriter->WriteRawJSONValue(TEXT("syncStatus"), SyncStatusJsonString);
	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();

	return Message;
}


FString CreateReceiveFileFromClientCompletedMessage(const FString& InDestinationPath)
{
	return CreateMessage(TEXT("send file complete"), true, { { TEXT("destination"), InDestinationPath } });
}

FString CreateReceiveFileFromClientFailedMessage(const FString& InDestinationPath, const FString& InError)
{
	return CreateMessage(TEXT("send file complete"), false, { { TEXT("destination"), InDestinationPath }, { TEXT("error"), InError } });
}

FString CreateSendFileToClientCompletedMessage(const FString& InSourcePath, const FString& InFileContent)
{
	return CreateMessage(TEXT("receive file complete"), true, { { TEXT("source"), InSourcePath }, { TEXT("content"), InFileContent } });
}

FString CreateSendFileToClientFailedMessage(const FString& InSourcePath, const FString& InError)
{
	return CreateMessage(TEXT("receive file complete"), false, { { TEXT("source"), InSourcePath }, { TEXT("error"), InError } });
}

FString CreateRedeployStatusMessage(const FGuid& InMessageID, bool bAck, const FString& Status)
{
	return CreateMessage(TEXT("redeploy server status"), bAck, { { TEXT("id"), InMessageID.ToString() }, { TEXT("status"), Status } });
}

FString CreateLiveLogStreamTerminatedMessage(const FString& InLogFilePath, const FString& InReason)
{
	return CreateMessage(TEXT("live log stream terminated"), true, {
		{ TEXT("log_file_path"), InLogFilePath },
		{ TEXT("reason"), InReason }
	});
}

// This logs an error with some context in the event the field is missing.
TSharedPtr<FJsonValue> TryGetCommandRequiredField(const TSharedPtr<FJsonObject>& CommandObj, const FString& FieldName)
{
	TSharedPtr<FJsonValue> Field = CommandObj->TryGetField(FieldName);
	if (!Field)
	{
		UE_LOGF(LogSwitchboardProtocol, Error, "\"%ls\" command missing required field \"%ls\"",
			*CommandObj->GetStringField(TEXT("command")).ToLower(), *FieldName);
	}
	return Field;
}


FCreateTaskResult CreateTaskFromCommand(const FString& InCommand, const FIPv4Endpoint& InEndpoint, bool bAuthenticated)
{
	FCreateTaskResult Result;

	TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(InCommand);
	TSharedPtr<FJsonObject> JsonData;
	if (!FJsonSerializer::Deserialize(Reader, JsonData))
	{
		Result.Status = ECreateTaskStatus::Error_ParsingFailed;
		return Result;
	}

	TSharedPtr<FJsonValue> CommandField = JsonData->TryGetField(TEXT("command"));
	TSharedPtr<FJsonValue> IdField = JsonData->TryGetField(TEXT("id"));

	if (!CommandField.IsValid())
	{
		Result.Status = ECreateTaskStatus::Error_ParsingFailed;
		return Result;
	}
	
	const FString CommandName = CommandField->AsString().ToLower();
	Result.CommandName = CommandName;

	FGuid MessageID;
	if (!IdField.IsValid() || !FGuid::Parse(IdField->AsString(), MessageID))
	{
		Result.Status = ECreateTaskStatus::Error_ParsingFailed;
		return Result;
	}

	Result.MessageID = MessageID;

	// Should we echo this command in the output log?
	{
		TSharedPtr<FJsonValue> EchoField = JsonData->TryGetField(TEXT("bEcho"));
		Result.bEcho = EchoField.IsValid() ? EchoField->AsBool() : true;
	}

	// Check commands allowed for unauthenticated clients first.
	if (CommandName == FSwitchboardDisconnectTask::CommandName)
	{
		Result.Status = ECreateTaskStatus::Success;
		Result.Task = MakeUnique<FSwitchboardDisconnectTask>(MessageID, InEndpoint);
		return Result;
	}
	else if (CommandName == FSwitchboardKeepAliveTask::CommandName)
	{
		Result.Status = ECreateTaskStatus::Success;
		Result.Task = MakeUnique<FSwitchboardKeepAliveTask>(MessageID, InEndpoint);
		return Result;
	}
	else if (CommandName == FSwitchboardSetInactiveTimeoutTask::CommandName)
	{
		TSharedPtr<FJsonValue> SecondsField = TryGetCommandRequiredField(JsonData, TEXT("seconds"));

		if (SecondsField.IsValid())
		{
			Result.Status = ECreateTaskStatus::Success;
			Result.Task = MakeUnique<FSwitchboardSetInactiveTimeoutTask>(MessageID, InEndpoint, SecondsField->AsNumber());
			return Result;
		}
	}
	else if (CommandName == FSwitchboardAuthenticateTask::CommandName)
	{
		// TODO: Delete me sometime after SBL 3.1. Not worth the breaking change.
		const FStringView DeprecatedTokenFieldName = TEXTVIEW("token");
		const bool bHasDeprecatedTokenField = JsonData->HasTypedField(DeprecatedTokenFieldName, EJson::String);

		const FStringView JwtFieldName = TEXTVIEW("jwt");
		const FStringView PasswordFieldName = TEXTVIEW("password");
		const bool bHasJwtField = JsonData->HasTypedField(JwtFieldName, EJson::String);
		const bool bHasPasswordField = JsonData->HasTypedField(PasswordFieldName, EJson::String);
		if (!bHasDeprecatedTokenField && !bHasJwtField && !bHasPasswordField)
		{
			Result.Status = ECreateTaskStatus::Error_ParsingFailed;
			return Result;
		}

		TOptional<FString> JwtField;
		TOptional<FString> PasswordField;

		if (bHasJwtField)
		{
			JwtField = JsonData->GetStringField(JwtFieldName);
		}

		if (bHasPasswordField)
		{
			PasswordField = JsonData->GetStringField(PasswordFieldName);
		}
		else if (bHasDeprecatedTokenField)
		{
			PasswordField = JsonData->GetStringField(DeprecatedTokenFieldName);
		}

		Result.Status = ECreateTaskStatus::Success;
		Result.Task = MakeUnique<FSwitchboardAuthenticateTask>(MessageID, InEndpoint, JwtField, PasswordField);
		return Result;
	}

	// These commands require authentication.
	if (!bAuthenticated)
	{
		Result.Status = ECreateTaskStatus::Error_Unauthenticated;
		return Result;
	}

	if (CommandName == FSwitchboardStartTask::CommandName)
	{
		TSharedPtr<FJsonValue> ExeField = TryGetCommandRequiredField(JsonData, TEXT("exe"));
		TSharedPtr<FJsonValue> ArgsField = TryGetCommandRequiredField(JsonData, TEXT("args"));
		TSharedPtr<FJsonValue> NameField = TryGetCommandRequiredField(JsonData, TEXT("name"));
		TSharedPtr<FJsonValue> CallerField = TryGetCommandRequiredField(JsonData, TEXT("caller"));
		TSharedPtr<FJsonValue> WorkingDirField = TryGetCommandRequiredField(JsonData, TEXT("working_dir"));

		if (!ExeField || !ArgsField || !NameField || !CallerField || !WorkingDirField)
		{
			Result.Status = ECreateTaskStatus::Error_ParsingFailed;
			return Result;
		}

		TUniquePtr<FSwitchboardStartTask> Task = MakeUnique<FSwitchboardStartTask>(
			MessageID,
			InEndpoint,
			ExeField->AsString(),
			ArgsField->AsString(),
			NameField->AsString(),
			CallerField->AsString(),
			WorkingDirField->AsString()
		);

		if (TSharedPtr<FJsonValue> UpdateClientsWithStdoutField = JsonData->TryGetField(TEXT("bUpdateClientsWithStdout")))
		{
			UpdateClientsWithStdoutField->TryGetBool(Task->bUpdateClientsWithStdout);
		}

		if (TSharedPtr<FJsonValue> CombinedStdoutStderrField = JsonData->TryGetField(TEXT("bCombinedStdoutStderr")))
		{
			CombinedStdoutStderrField->TryGetBool(Task->bCombinedStdoutStderr);
		}

		if (TSharedPtr<FJsonValue> LockGpuClock = JsonData->TryGetField(TEXT("bLockGpuClock")))
		{
			LockGpuClock->TryGetBool(Task->bLockGpuClock);
		}

		if (TSharedPtr<FJsonValue> PriorityModifierField = JsonData->TryGetField(TEXT("priority_modifier")))
		{
			PriorityModifierField->TryGetNumber(Task->PriorityModifier);
		}

		if (TSharedPtr<FJsonValue> HideField = JsonData->TryGetField(TEXT("bHide")))
		{
			HideField->TryGetBool(Task->bHide);
		}

		Result.Status = ECreateTaskStatus::Success;
		Result.Task = MoveTemp(Task);
		return Result;
	}
	else if (CommandName == FSwitchboardKillTask::CommandName)
	{
		TSharedPtr<FJsonValue> UUIDField = TryGetCommandRequiredField(JsonData, TEXT("uuid"));

		FGuid ProgramID;
		if (UUIDField.IsValid() && FGuid::Parse(UUIDField->AsString(), ProgramID))
		{
			// Extract optional soft kill timeout, use default if not provided
			TUniquePtr<FSwitchboardKillTask> KillTask = MakeUnique<FSwitchboardKillTask>(MessageID, InEndpoint, ProgramID);
			if (TSharedPtr<FJsonValue> TimeoutField = JsonData->TryGetField(TEXT("soft_kill_timeout")))
			{
				KillTask->SoftKillTimeout = TimeoutField->AsNumber();
			}

			Result.Status = ECreateTaskStatus::Success;
			Result.Task = MoveTemp(KillTask);
			return Result;
		}
	}
	else if (CommandName == FSwitchboardReceiveFileFromClientTask::CommandName)
	{
		TSharedPtr<FJsonValue> DestinationField = TryGetCommandRequiredField(JsonData, TEXT("destination"));
		TSharedPtr<FJsonValue> FileContentField = TryGetCommandRequiredField(JsonData, TEXT("content"));
		
		if (!DestinationField || !FileContentField)
		{
			Result.Status = ECreateTaskStatus::Error_ParsingFailed;
			return Result;
		}

		TUniquePtr<FSwitchboardReceiveFileFromClientTask> Task = MakeUnique<FSwitchboardReceiveFileFromClientTask>(
			MessageID,
			InEndpoint,
			DestinationField->AsString(),
			FileContentField->AsString()
		);

		if (TSharedPtr<FJsonValue> ForceOverwriteField = JsonData->TryGetField(TEXT("force_overwrite")))
		{
			ForceOverwriteField->TryGetBool(Task->bForceOverwrite);
		}

		Result.Status = ECreateTaskStatus::Success;
		Result.Task = MoveTemp(Task);
		return Result;
	}
	else if (CommandName == FSwitchboardSendFileToClientTask::CommandName)
	{
		TSharedPtr<FJsonValue> SourceField = TryGetCommandRequiredField(JsonData, TEXT("source"));

		if (SourceField.IsValid())
		{
			Result.Status = ECreateTaskStatus::Success;
			Result.Task = MakeUnique<FSwitchboardSendFileToClientTask>(MessageID, InEndpoint, SourceField->AsString());
			return Result;
		}
	}
	else if (CommandName == FSwitchboardGetSyncStatusTask::CommandName)
	{
		TSharedPtr<FJsonValue> UUIDField = TryGetCommandRequiredField(JsonData, TEXT("uuid"));

		if (!UUIDField.IsValid())
		{
			Result.Status = ECreateTaskStatus::Error_ParsingFailed;
			return Result;
		}

		FGuid ProgramID;

		if (!FGuid::Parse(UUIDField->AsString(), ProgramID))
		{
			Result.Status = ECreateTaskStatus::Error_ParsingFailed;
			return Result;
		}

		TSharedPtr<FJsonValue> RequestFlagsField = TryGetCommandRequiredField(JsonData, TEXT("request_flags"));

		if (!RequestFlagsField.IsValid())
		{
			Result.Status = ECreateTaskStatus::Error_ParsingFailed;
			return Result;
		}

		using RequestFlagsIntType = std::underlying_type<ESyncStatusRequestFlags>::type;
		RequestFlagsIntType RequestFlags;

		if (!RequestFlagsField->TryGetNumber(RequestFlags))
		{
			Result.Status = ECreateTaskStatus::Error_ParsingFailed;
			return Result;
		}

		const ESyncStatusRequestFlags RequestFlagsEnum = static_cast<ESyncStatusRequestFlags>(RequestFlags);

		Result.Status = ECreateTaskStatus::Success;
		Result.Task = MakeUnique<FSwitchboardGetSyncStatusTask>(MessageID, InEndpoint, ProgramID, RequestFlagsEnum);

		return Result;
	}
	else if (CommandName == FSwitchboardRedeployListenerTask::CommandName)
	{		
		TSharedPtr<FJsonValue> Sha1Field = TryGetCommandRequiredField(JsonData, TEXT("sha1"));
		TSharedPtr<FJsonValue> FileContentField = TryGetCommandRequiredField(JsonData, TEXT("content"));

		if (Sha1Field.IsValid() && FileContentField.IsValid())
		{
			Result.Status = ECreateTaskStatus::Success;
			Result.Task = MakeUnique<FSwitchboardRedeployListenerTask>(MessageID, InEndpoint, Sha1Field->AsString(), FileContentField->AsString());
			return Result;
		}
	}
	else if (CommandName == FSwitchboardFreeListenerBinaryTask::CommandName)
	{
		Result.Status = ECreateTaskStatus::Success;
		Result.Task = MakeUnique<FSwitchboardFreeListenerBinaryTask>(MessageID, InEndpoint);
		return Result;
	}
	else if (CommandName == FSwitchboardFixExeFlagsTask::CommandName)
	{
		TSharedPtr<FJsonValue> UUIDField = TryGetCommandRequiredField(JsonData, TEXT("uuid"));

		FGuid ProgramID;
		if (UUIDField.IsValid() && FGuid::Parse(UUIDField->AsString(), ProgramID))
		{
			Result.Status = ECreateTaskStatus::Success;
			Result.Task = MakeUnique<FSwitchboardFixExeFlagsTask>(MessageID, InEndpoint, ProgramID);
			return Result;
		}
	}
	else if (CommandName == FSwitchboardRefreshMosaicsTask::CommandName)
	{
		Result.Status = ECreateTaskStatus::Success;
		Result.Task = MakeUnique<FSwitchboardRefreshMosaicsTask>(MessageID, InEndpoint);
		return Result;
	}
	else if (CommandName == FSwitchboardMinimizeWindowsTask::CommandName)
	{
		Result.Status = ECreateTaskStatus::Success;
		Result.Task = MakeUnique<FSwitchboardMinimizeWindowsTask>(MessageID, InEndpoint);
		return Result;
	}
	else if (CommandName == FSwitchboardStartLiveLogStreamTask::CommandName)
	{
		TSharedPtr<FJsonValue> LogFilePathField = TryGetCommandRequiredField(JsonData, TEXT("log_file_path"));

		if (LogFilePathField.IsValid())
		{
			// Optional tail_only field

			bool bTailOnly = false; // Default to send from the beginning

			TSharedPtr<FJsonValue> TailOnlyField = JsonData->TryGetField(TEXT("tail_only"));

			if (TailOnlyField.IsValid())
			{
				bTailOnly = TailOnlyField->AsBool();
			}

			// Optional process_uuid field (currently there always is one)

			FGuid ProcessUUID;
			TSharedPtr<FJsonValue> ProcessUUIDField = JsonData->TryGetField(TEXT("process_uuid"));
			if (ProcessUUIDField.IsValid())
			{
				FGuid::Parse(ProcessUUIDField->AsString(), ProcessUUID);
			}

			// Optional start_offset field for resuming log streaming

			int64 StartOffset = -1; // -1 means use default behavior
			TSharedPtr<FJsonValue> StartOffsetField = JsonData->TryGetField(TEXT("start_offset"));
			if (StartOffsetField.IsValid())
			{
				StartOffset = (int64)StartOffsetField->AsNumber();
			}

			Result.Status = ECreateTaskStatus::Success;
			Result.Task = MakeUnique<FSwitchboardStartLiveLogStreamTask>(MessageID, InEndpoint, LogFilePathField->AsString(), bTailOnly, ProcessUUID, StartOffset);
			return Result;
		}
	}
	else if (CommandName == FSwitchboardStopLiveLogStreamTask::CommandName)
	{
		TSharedPtr<FJsonValue> LogFilePathField = TryGetCommandRequiredField(JsonData, TEXT("log_file_path"));

		if (LogFilePathField.IsValid())
		{
			Result.Status = ECreateTaskStatus::Success;
			Result.Task = MakeUnique<FSwitchboardStopLiveLogStreamTask>(MessageID, InEndpoint, LogFilePathField->AsString());
			return Result;
		}
	}

	Result.Status = ECreateTaskStatus::Error_Unhandled;
	return Result;
}
