// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "JsonObjectConverter.h"
#include "Logging/LogMacros.h"
#include "SwitchboardListenerApp.h"


struct FSwitchboardPacket;
struct FSwitchboardTask;
struct FSyncStatus;
enum class ESyncStatusRequestFlags;


//~ Messages sent from Listener to Switchboard

template<typename InStructType>
FString CreateMessage(const InStructType& InStruct)
{
	FString Message;
	const bool bMessageOk = FJsonObjectConverter::UStructToJsonObjectString(InStruct, Message);
	check(bMessageOk);
	return Message;
}
FString CreateMessage(const TMap<FString, FString>& InFields);

FString CreateTaskDeclinedMessage(const FSwitchboardTask& InTask, const FString& InErrorMessage, const TMap<FString, FString>& InAdditionalFields);
FString CreateCommandAcceptedMessage(const FGuid& InMessageID);
FString CreateCommandDeclinedMessage(const FGuid& InMessageID, const FString& InErrorMessage);

FString CreateReceiveFileFromClientCompletedMessage(const FString& InDestinationPath);
FString CreateReceiveFileFromClientFailedMessage(const FString& InDestinationPath, const FString& InError);

FString CreateSendFileToClientCompletedMessage(const FString& InSourcePath, const FString& InFileContent);
FString CreateSendFileToClientFailedMessage(const FString& InSourcePath, const FString& InError);

FString CreateSyncStatusMessage(const FSyncStatus& SyncStatus, ESyncStatusRequestFlags RequestFlags);

FString CreateRedeployStatusMessage(const FGuid& InMessageID, bool bAck, const FString& Status);

FString CreateLiveLogStreamTerminatedMessage(const FString& InLogFilePath, const FString& InReason);

//~

enum class ECreateTaskStatus
{
	Success,

	Error_Unauthenticated,
	Error_ParsingFailed,
	Error_Unhandled,
};

struct FCreateTaskResult
{
	ECreateTaskStatus Status;
	TUniquePtr<FSwitchboardTask> Task;
	bool bEcho;

	TOptional<FString> CommandName;
	TOptional<FGuid> MessageID;
};

FCreateTaskResult CreateTaskFromCommand(const FString& InCommand, const FIPv4Endpoint& InEndpoint, bool bAuthenticated);
