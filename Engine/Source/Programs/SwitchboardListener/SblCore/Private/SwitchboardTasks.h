// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Misc/SecureHash.h"
#include "Templates/TypeHash.h"

enum class ESwitchboardTaskType : uint8
{
	Authenticate,
	Start,
	Kill,
	Restart,
	ReceiveFileFromClient,
	SendFileToClient,
	Disconnect,
	KeepAlive,
	GetSyncStatus,
	GetFlipMode,
	FixExeFlags,
	RedeployListener,
	RefreshMosaics,
	MinimizeWindows,
	SetInactiveTimeout,
	FreeListenerBinary,
	StartLiveLogStream,
	StopLiveLogStream
};


struct FSwitchboardTask
{
	ESwitchboardTaskType Type;
	FGuid TaskID;
	FIPv4Endpoint Recipient;

	FSwitchboardTask(ESwitchboardTaskType InType, FGuid InTaskID, FIPv4Endpoint InRecipient)
		: Type(InType)
		, TaskID(InTaskID)
		, Recipient(InRecipient)
	{}

	virtual ~FSwitchboardTask() = default;

	virtual const TCHAR* GetCommandName() const = 0;

	/** Calculates a hash that should be the same for equivalent Tasks, even if their TaskID is different */
	virtual uint32 GetEquivalenceHash() const
	{
		return HashCombine(GetTypeHash(Type), GetTypeHash(Recipient));
	}	
};

struct FSwitchboardAuthenticateTask : public FSwitchboardTask
{
	FSwitchboardAuthenticateTask(
		const FGuid& InTaskId,
		const FIPv4Endpoint& InEndpoint,
		TOptional<FString> InJwt,
		TOptional<FString> InPassword
	)
		: FSwitchboardTask{ ESwitchboardTaskType::Authenticate, InTaskId, InEndpoint }
		, Jwt(InJwt)
		, Password(InPassword)
	{
		ensure(Jwt || Password);
	}

	TOptional<FString> Jwt;
	TOptional<FString> Password;

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("authenticate");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };

	virtual uint32 GetEquivalenceHash() const override
	{
		return HashCombine(FSwitchboardTask::GetEquivalenceHash(), GetTypeHash(Password));
	}
	//~ End FSwitchboardTask interface
};

/** Bit flags for the request. It should match its counterpart's definition in message_protocol.py */
enum class ESyncStatusRequestFlags
{
	None = 0,
	SyncTopos = 1 << 0,
	MosaicTopos = 1 << 1,
	FlipModeHistory = 1 << 2,
	ProgramLayers = 1 << 3,
	DriverInfo = 1 << 4,
	Taskbar = 1 << 5,
	PidInFocus = 1 << 6,
	CpuUtilization = 1 << 7,
	AvailablePhysicalMemory = 1 << 8,
	GpuUtilization = 1 << 9,
	GpuCoreClockKhz = 1 << 10,
	GpuTemperature = 1 << 11,
};

struct FSwitchboardGetSyncStatusTask : public FSwitchboardTask
{
	FSwitchboardGetSyncStatusTask(
		const FGuid& InTaskId, 
		const FIPv4Endpoint& InEndpoint, 
		const FGuid& InProgramID, 
		const ESyncStatusRequestFlags InRequestFlags
	)
		: FSwitchboardTask{ ESwitchboardTaskType::GetSyncStatus, InTaskId, InEndpoint }
		, ProgramID(InProgramID)
		, RequestFlags(InRequestFlags)
	{}

	/** ID of the program that we wish to get the FlipMode of */
	FGuid ProgramID;

	/** Mask for the information requested */
	ESyncStatusRequestFlags RequestFlags = static_cast<ESyncStatusRequestFlags>(~0);

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("get sync status");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };

	virtual uint32 GetEquivalenceHash() const override
	{
		uint32 Hash = HashCombine(FSwitchboardTask::GetEquivalenceHash(), GetTypeHash(ProgramID));
		return HashCombine(Hash, GetTypeHash(RequestFlags));
	}
	//~ End FSwitchboardTask interface
};

struct FSwitchboardRefreshMosaicsTask : public FSwitchboardTask
{
	FSwitchboardRefreshMosaicsTask(const FGuid& InTaskId, const FIPv4Endpoint& InEndpoint)
		: FSwitchboardTask{ ESwitchboardTaskType::RefreshMosaics, InTaskId, InEndpoint }
	{}

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("refresh mosaics");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };
	//~ End FSwitchboardTask interface
};

struct FSwitchboardStartTask : public FSwitchboardTask
{
	FSwitchboardStartTask(
		const FGuid& InTaskId, 
		const FIPv4Endpoint& InEndpoint, 
		const FString& InCommand, 
		const FString& InArgs, 
		const FString& InName, 
		const FString& InCaller, 
		const FString& InWorkingDir
	)
		: FSwitchboardTask{ ESwitchboardTaskType::Start, InTaskId, InEndpoint }
		, Command(InCommand)
		, Arguments(InArgs)
		, Name(InName)
		, Caller(InCaller)
		, WorkingDir(InWorkingDir)
	{
	}

	FString Command;
	FString Arguments;
	FString Name;
	FString Caller;
	FString WorkingDir;
	bool bUpdateClientsWithStdout = false;
	bool bCombinedStdoutStderr = true;
	bool bLockGpuClock = false;
	int32 PriorityModifier = 0;
	bool bHide = false;

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("start");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };

	virtual uint32 GetEquivalenceHash() const override
	{
		uint32 Hash = HashCombine(FSwitchboardTask::GetEquivalenceHash(), GetTypeHash(Command));
		Hash = HashCombine(Hash, GetTypeHash(Caller));
		return HashCombine(Hash, GetTypeHash(Arguments));
	}
	//~ End FSwitchboardTask interface
};

struct FSwitchboardKillTask : public FSwitchboardTask
{
	FSwitchboardKillTask(const FGuid& InTaskId, const FIPv4Endpoint& InEndpoint, const FGuid& InProgramID, float InSoftKillTimeout = -1.0f)
		: FSwitchboardTask{ ESwitchboardTaskType::Kill, InTaskId, InEndpoint}
		, ProgramID(InProgramID)
		, SoftKillTimeout(InSoftKillTimeout)
	{}

	// unique ID of process to kill
	FGuid ProgramID;

	/** Time after a soft termination signal before issuing a hard termination signal. */
	float SoftKillTimeout = 2.0f;

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("kill");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };

	virtual uint32 GetEquivalenceHash() const override
	{
		// Ignore the SoftKillTimeout for this.  A new kill command of the same program with a different 
		// timeout should be considered a duplicate.
		return HashCombine(FSwitchboardTask::GetEquivalenceHash(), GetTypeHash(ProgramID));
	}
	//~ End FSwitchboardTask interface
};

struct FSwitchboardReceiveFileFromClientTask : public FSwitchboardTask
{
	FSwitchboardReceiveFileFromClientTask(const FGuid& InTaskID, const FIPv4Endpoint& InEndpoint, const FString& InDestination, const FString& InContent)
		: FSwitchboardTask{ ESwitchboardTaskType::ReceiveFileFromClient, InTaskID, InEndpoint }
		, Destination(InDestination)
		, FileContent(InContent)
	{}

	FString Destination;
	FString FileContent;
	bool bForceOverwrite = false;

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("send file");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };

	virtual uint32 GetEquivalenceHash() const override
	{
		uint32 Hash = HashCombine(FSwitchboardTask::GetEquivalenceHash(), GetTypeHash(Destination));
		return HashCombine(Hash, GetTypeHash(FileContent));
	}
	//~ End FSwitchboardTask interface
};

struct FSwitchboardRedeployListenerTask : public FSwitchboardTask
{
	FSwitchboardRedeployListenerTask(const FGuid& InTaskID, const FIPv4Endpoint& InEndpoint, const FString& InExpectedHashHexDigest, const FString& InContent)
		: FSwitchboardTask{ ESwitchboardTaskType::RedeployListener, InTaskID, InEndpoint }
		, FileContent(InContent)
	{
		ExpectedHash.FromString(InExpectedHashHexDigest);
	}

	FSHAHash ExpectedHash;
	FString FileContent;

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("redeploy listener");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };

	virtual uint32 GetEquivalenceHash() const override
	{
		uint32 Hash = HashCombine(FSwitchboardTask::GetEquivalenceHash(), GetTypeHash(ExpectedHash));
		return HashCombine(Hash, GetTypeHash(FileContent));
	}
	//~ End FSwitchboardTask interface
};

struct FSwitchboardSendFileToClientTask : public FSwitchboardTask
{
	FSwitchboardSendFileToClientTask(const FGuid& InTaskID, const FIPv4Endpoint& InEndpoint, const FString& InSource)
		: FSwitchboardTask{ ESwitchboardTaskType::SendFileToClient, InTaskID, InEndpoint }
		, Source(InSource)
	{}

	FString Source;

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("receive file");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };

	virtual uint32 GetEquivalenceHash() const override
	{
		return HashCombine(FSwitchboardTask::GetEquivalenceHash(), GetTypeHash(Source));
	}
	//~ End FSwitchboardTask interface
};

struct FSwitchboardFixExeFlagsTask : public FSwitchboardTask
{
	FSwitchboardFixExeFlagsTask(const FGuid& InTaskId, const FIPv4Endpoint& InEndpoint, const FGuid& InProgramID)
		: FSwitchboardTask{ ESwitchboardTaskType::FixExeFlags, InTaskId, InEndpoint }
		, ProgramID(InProgramID)
	{}

	FGuid ProgramID;

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("fixExeFlags");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };

	virtual uint32 GetEquivalenceHash() const override
	{
		return HashCombine(FSwitchboardTask::GetEquivalenceHash(), GetTypeHash(ProgramID));
	}
	//~ End FSwitchboardTask interface
};

struct FSwitchboardDisconnectTask : public FSwitchboardTask
{
	FSwitchboardDisconnectTask(const FGuid& InTaskId, const FIPv4Endpoint& InEndpoint)
		: FSwitchboardTask{ESwitchboardTaskType::Disconnect, InTaskId, InEndpoint}
	{}

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("disconnect");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };
	//~ End FSwitchboardTask interface
};

struct FSwitchboardKeepAliveTask : public FSwitchboardTask
{
	FSwitchboardKeepAliveTask(const FGuid& InTaskId, const FIPv4Endpoint& InEndpoint)
		: FSwitchboardTask{ESwitchboardTaskType::KeepAlive, InTaskId, InEndpoint}
	{}

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("keep alive");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };
	//~ End FSwitchboardTask interface
};

struct FSwitchboardMinimizeWindowsTask : public FSwitchboardTask
{
	FSwitchboardMinimizeWindowsTask(const FGuid& InTaskId, const FIPv4Endpoint& InEndpoint)
		: FSwitchboardTask{ ESwitchboardTaskType::MinimizeWindows, InTaskId, InEndpoint }
	{}

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("minimize windows");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };
	//~ End FSwitchboardTask interface
};

struct FSwitchboardSetInactiveTimeoutTask : public FSwitchboardTask
{
	FSwitchboardSetInactiveTimeoutTask(const FGuid& InTaskId, const FIPv4Endpoint& InEndpoint, double InTimeoutSeconds)
		: FSwitchboardTask{ ESwitchboardTaskType::SetInactiveTimeout, InTaskId, InEndpoint }
		, TimeoutSeconds(InTimeoutSeconds)
	{}

	double TimeoutSeconds;

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("set inactive timeout");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };

	virtual uint32 GetEquivalenceHash() const override
	{
		return HashCombine(FSwitchboardTask::GetEquivalenceHash(), GetTypeHash(TimeoutSeconds));
	}
	//~ End FSwitchboardTask interface
};

struct FSwitchboardFreeListenerBinaryTask : public FSwitchboardTask
{
	FSwitchboardFreeListenerBinaryTask(const FGuid& InTaskID, const FIPv4Endpoint& InEndpoint)
		: FSwitchboardTask{ ESwitchboardTaskType::FreeListenerBinary, InTaskID, InEndpoint }
	{}

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("free binary");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };
	//~ End FSwitchboardTask interface
};

struct FSwitchboardStartLiveLogStreamTask : public FSwitchboardTask
{
	FSwitchboardStartLiveLogStreamTask(const FGuid& InTaskID, const FIPv4Endpoint& InEndpoint, const FString& InLogFilePath, bool bInTailOnly = false, const FGuid& InProcessUUID = FGuid(), int64 InStartOffset = -1)
		: FSwitchboardTask{ ESwitchboardTaskType::StartLiveLogStream, InTaskID, InEndpoint }
		, LogFilePath(InLogFilePath)
		, bTailOnly(bInTailOnly)
		, ProcessUUID(InProcessUUID)
		, StartOffset(InStartOffset)

	{}

	FString LogFilePath;
	bool bTailOnly;
	FGuid ProcessUUID;
	int64 StartOffset; // -1 means use default behavior (0 or file size based on bTailOnly)

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("start live log stream");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };

	virtual uint32 GetEquivalenceHash() const override
	{
		uint32 Hash = HashCombine(FSwitchboardTask::GetEquivalenceHash(), GetTypeHash(LogFilePath));
		return HashCombine(Hash, GetTypeHash(bTailOnly));
	}
	//~ End FSwitchboardTask interface
};

struct FSwitchboardStopLiveLogStreamTask : public FSwitchboardTask
{
	FSwitchboardStopLiveLogStreamTask(const FGuid& InTaskID, const FIPv4Endpoint& InEndpoint, const FString& InLogFilePath)
		: FSwitchboardTask{ ESwitchboardTaskType::StopLiveLogStream, InTaskID, InEndpoint }
		, LogFilePath(InLogFilePath)
	{}

	FString LogFilePath;

	//~ Begin FSwitchboardTask interface
	static constexpr const TCHAR* CommandName = TEXT("stop live log stream");
	virtual const TCHAR* GetCommandName() const override { return CommandName; };

	virtual uint32 GetEquivalenceHash() const override
	{
		return HashCombine(FSwitchboardTask::GetEquivalenceHash(), GetTypeHash(LogFilePath));
	}
	//~ End FSwitchboardTask interface
};
