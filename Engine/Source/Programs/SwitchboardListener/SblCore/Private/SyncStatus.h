// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SyncStatus.generated.h"


USTRUCT()
struct FSyncGpu
{
	GENERATED_BODY()

	UPROPERTY()
	bool bIsSynced;

	UPROPERTY()
	int32 Connector;
};

USTRUCT()
struct FSyncDisplay
{
	GENERATED_BODY()

	UPROPERTY()
	FName SyncState;

	UPROPERTY()
	uint32 Bpc;

	UPROPERTY()
	uint8 ColorFormat;

	UPROPERTY()
	int32 Depth;

};


USTRUCT()
struct FSyncStatusParams
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 RefreshRate;

	UPROPERTY()
	uint32 HouseSyncIncoming;

	UPROPERTY()
	bool bHouseSync;

	/** Per nvapi.h comment: "means that this P2061 board receives input from another P2061 board" */
	UPROPERTY()
	bool bInternalSecondary;
};

USTRUCT()
struct FSyncDelay
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 NumLines;

	UPROPERTY()
	uint32 NumPixels;

	UPROPERTY()
	uint32 MaxLines;

	UPROPERTY()
	uint32 MinPixels;
};

USTRUCT()
struct FSyncControlParams
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Polarity;

	UPROPERTY()
	int32 VMode;

	UPROPERTY()
	uint32 Interval;

	UPROPERTY()
	int32 Source;

	UPROPERTY()
	bool bInterlaced;

	UPROPERTY()
	bool bSyncSourceIsOutput;

	UPROPERTY()
	FSyncDelay SyncSkew;

	UPROPERTY()
	FSyncDelay StartupDelay;
};

USTRUCT()
struct FSyncTopo
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FSyncGpu> SyncGpus;

	UPROPERTY()
	TArray<FSyncDisplay> SyncDisplays;

	UPROPERTY()
	FSyncStatusParams SyncStatusParams;

	UPROPERTY()
	FSyncControlParams SyncControlParams;
};

USTRUCT()
struct FMosaicDisplaySettings
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 Width;

	UPROPERTY()
	uint32 Height;

	UPROPERTY()
	uint32 Bpp;

	UPROPERTY()
	uint32 Freq;
};

USTRUCT()
struct FMosaicTopo
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 Rows;

	UPROPERTY()
	uint32 Columns;

	UPROPERTY()
	uint32 DisplayCount;

	UPROPERTY()
	FMosaicDisplaySettings DisplaySettings;
};

USTRUCT()
struct FSyncStatus
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FSyncTopo> SyncTopos;

	UPROPERTY()
	TArray<FMosaicTopo> MosaicTopos;

	UPROPERTY()
	TArray<FString> FlipModeHistory;

	UPROPERTY()
	TArray<FString> ProgramLayers;

	UPROPERTY()
	uint32 DriverVersion;

	UPROPERTY()
	FString DriverBranch;

	UPROPERTY()
	FString Taskbar;

	UPROPERTY()
	uint32 PidInFocus;

	UPROPERTY()
	TArray<int8> CpuUtilization;

	UPROPERTY()
	uint64 AvailablePhysicalMemory;

	UPROPERTY()
	TArray<int8> GpuUtilization;

	UPROPERTY()
	TArray<int32> GpuCoreClocksKhz;

	UPROPERTY()
	TArray<int32> GpuTemperature;
};
