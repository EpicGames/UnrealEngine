// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsCpuUtilizationMonitor.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformTime.h"


FWindowsCpuUtilizationMonitor::FWindowsCpuUtilizationMonitor()
{
	const int32 NumCores = FPlatformMisc::NumberOfCores();

	CounterHandles.SetNum(NumCores);
	CachedCoreUtilization.SetNum(NumCores);

	PDH_STATUS PdhStatus = ::PdhOpenQuery(nullptr, 0, &QueryHandle);
	if (PdhStatus != ERROR_SUCCESS)
	{
		UE_LOGF(LogCpuUtilizationMonitor, Warning, "PdhOpenQuery failed. Error code: %d", PdhStatus);
		return;
	}

	for (int32 CoreIdx = 0; CoreIdx < NumCores; ++CoreIdx)
	{
		const FString CounterName = FString::Printf(TEXT("\\Processor(%d)\\%% Processor Time"), CoreIdx);
		PdhStatus = ::PdhAddEnglishCounter(QueryHandle, *CounterName, 0, &CounterHandles[CoreIdx]);
		if (PdhStatus != ERROR_SUCCESS)
		{
			UE_LOGF(LogCpuUtilizationMonitor, Warning, "PdhAddEnglishCounter failed. Error code: %d", PdhStatus);
			return;
		}
	}

	// Initial condition: ensures we've collected twice before the first attempt to get values.
	PdhStatus = ::PdhCollectQueryData(QueryHandle);
	if (PdhStatus != ERROR_SUCCESS)
	{
		UE_LOGF(LogCpuUtilizationMonitor, Warning, "PdhCollectQueryData failed. Error code: %d", PdhStatus);
		return;
	}

	bIsInitialized = true;
}

FWindowsCpuUtilizationMonitor::~FWindowsCpuUtilizationMonitor()
{
	if (QueryHandle != INVALID_HANDLE_VALUE)
	{
		PDH_STATUS PdhStatus = ::PdhCloseQuery(QueryHandle);
		if (PdhStatus != ERROR_SUCCESS)
		{
			UE_LOGF(LogCpuUtilizationMonitor, Warning, "PdhCloseQuery failed. Error code: %d", PdhStatus);
		}
	}
}

bool FWindowsCpuUtilizationMonitor::GetPerCoreUtilization(TArray<int8>& OutCoreUtilization)
{
	if (!IsInitialized())
	{
		return false;
	}

	// Rate limiting. MSDN recommends <= 1 Hz.
	const double MinQueryIntervalSec = 1.0;
	const double TimeNow = FPlatformTime::Seconds();
	if (TimeNow - LastQueryTime >= MinQueryIntervalSec)
	{
		if (!QueryUpdatedUtilization())
		{
			return false;
		}
	}

	OutCoreUtilization = CachedCoreUtilization;

	return true;
}

bool FWindowsCpuUtilizationMonitor::QueryUpdatedUtilization()
{
	PDH_STATUS PdhStatus = ::PdhCollectQueryData(QueryHandle);
	if (PdhStatus != ERROR_SUCCESS)
	{
		UE_LOGF(LogCpuUtilizationMonitor, Warning, "PdhCollectQueryData failed. Error code: %d", PdhStatus);
		return false;
	}

	const int32 NumHandles = CounterHandles.Num();
	for (int32 HandleIdx = 0; HandleIdx < NumHandles; ++HandleIdx)
	{
		// Use PDH_FMT_DOULE here because PDH_FMT_LONG truncates instead of rounding.
		PDH_FMT_COUNTERVALUE CounterValue;
		PdhStatus = ::PdhGetFormattedCounterValue(CounterHandles[HandleIdx], PDH_FMT_DOUBLE, 0, &CounterValue);
		if (PdhStatus != ERROR_SUCCESS)
		{
			UE_LOGF(LogCpuUtilizationMonitor, Warning, "PdhGetFormattedCounterValue failed. Error code: %d", PdhStatus);
			return false;
		}

		CachedCoreUtilization[HandleIdx] = FMath::RoundToInt(CounterValue.doubleValue);
	}

	LastQueryTime = FPlatformTime::Seconds();

	return true;
}
