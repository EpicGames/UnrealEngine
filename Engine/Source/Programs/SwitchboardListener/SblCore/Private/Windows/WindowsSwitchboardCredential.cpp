// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsSwitchboardCredential.h"
#include "SwitchboardListenerApp.h" // for LogSwitchboard

#include "Logging/StructuredLog.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <wincred.h>
#include "Windows/HideWindowsPlatformTypes.h"


namespace UE::SwitchboardListener::Private
{
	class FWindowsCredential : public ICredential
	{
	public:
		FWindowsCredential(CREDENTIALW* InNativeCredential)
			: NativeCredential(InNativeCredential)
		{
			if (ensure(InNativeCredential))
			{
				UserView = FStringView(NativeCredential->UserName);
				BlobView = FStringView(
					reinterpret_cast<TCHAR*>(NativeCredential->CredentialBlob),
					NativeCredential->CredentialBlobSize / sizeof(TCHAR));
			}
		}

		virtual FStringView GetUser() override
		{
			return UserView;
		}


		virtual FStringView GetBlob() override
		{
			return BlobView;
		}

		virtual ~FWindowsCredential() override
		{
			if (ensure(NativeCredential))
			{
				::CredFree(NativeCredential);
				NativeCredential = nullptr;
			}
		}

	private:
		CREDENTIALW* NativeCredential;
		FStringView UserView;
		FStringView BlobView;
	};

	FString GetCredentialTargetName(FStringView InCredentialName)
	{
		return FString::Printf(TEXT("SwitchboardListener_%*s"),
			InCredentialName.Len(), InCredentialName.GetData());
	}

	TSharedPtr<ICredential> FWindowsCredentialManager::LoadCredential(FStringView InCredentialName)
	{
		const FString TargetName = GetCredentialTargetName(InCredentialName);

		CREDENTIALW* OutCredPtr = nullptr;
		if (::CredReadW(*TargetName, CRED_TYPE_GENERIC, 0, &OutCredPtr))
		{
			return MakeShared<FWindowsCredential>(OutCredPtr);
		}
		else
		{
			const int64 LastError = ::GetLastError();
			if (LastError == ERROR_NOT_FOUND)
			{
				UE_LOGFMT(LogSwitchboard, Verbose, "CredReadW returned ERROR_NOT_FOUND");
			}
			else
			{
				UE_LOGFMT(LogSwitchboard, Warning, "CredReadW failed; GetLastError = {LastError}",
					static_cast<int64>(::GetLastError()));
			}
		}

		return {};
	}

	bool FWindowsCredentialManager::SaveCredential(FStringView InCredentialName, FStringView InUser, FStringView InBlob)
	{
		const FString TargetName = GetCredentialTargetName(InCredentialName);
		const FString UserName(InUser);

		CREDENTIALW Cred = { 0 };
		Cred.Type = CRED_TYPE_GENERIC;
		Cred.TargetName = const_cast<TCHAR*>(*TargetName);
		Cred.CredentialBlobSize = sizeof(TCHAR) * InBlob.Len();
		Cred.CredentialBlob = reinterpret_cast<BYTE*>(
			const_cast<TCHAR*>(InBlob.GetData()));
		Cred.Persist = CRED_PERSIST_LOCAL_MACHINE;
		Cred.UserName = const_cast<TCHAR*>(*UserName);

		if (!::CredWriteW(&Cred, 0))
		{
			UE_LOGFMT(LogSwitchboard, Warning, "CredWriteW failed; GetLastError = {LastError}",
				static_cast<int64>(::GetLastError()));

			return false;
		}

		return true;
	}
}; // namespace UE::SwitchboardListener::Private
