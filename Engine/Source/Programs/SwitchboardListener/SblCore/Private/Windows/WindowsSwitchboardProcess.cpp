// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsSwitchboardProcess.h"
#include "Misc/ScopeExit.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#	include <processthreadsapi.h>
#include "Windows/HideWindowsPlatformTypes.h"


namespace UE::SwitchboardListener::Private
{
	void LogLastError(const TCHAR* ErrorContext)
	{
		const DWORD ErrorCode = GetLastError();
		TCHAR ErrorMessage[512];
		FWindowsPlatformMisc::GetSystemErrorMessage(ErrorMessage, 512, ErrorCode);
		UE_LOGF(LogWindows, Warning, "%ls failed: %ls (0x%08x)", ErrorContext, ErrorMessage, ErrorCode);
	}
}


// This is required because nested UpdateProcThreadAttribute macros embed TRUE/FALSE :|
#include "Windows/AllowWindowsPlatformTypes.h"

FWindowsSwitchboardProcess::FCreateProcResult FWindowsSwitchboardProcess::CreateProc(const FCreateProcParams& InParams)
{
	// initialize process creation flags
	uint32 CreateFlags = NORMAL_PRIORITY_CLASS;
	if (InParams.PriorityModifier < 0)
	{
		CreateFlags = (InParams.PriorityModifier == -1) ? BELOW_NORMAL_PRIORITY_CLASS : IDLE_PRIORITY_CLASS;
	}
	else if (InParams.PriorityModifier > 0)
	{
		CreateFlags = (InParams.PriorityModifier == 1) ? ABOVE_NORMAL_PRIORITY_CLASS : HIGH_PRIORITY_CLASS;
	}

	if (InParams.bLaunchDetached)
	{
		CreateFlags |= DETACHED_PROCESS;
	}

	// initialize window flags
	uint32 dwFlags = 0;
	uint16 ShowWindowFlags = SW_HIDE;
	if (InParams.bLaunchReallyHidden)
	{
		dwFlags = STARTF_USESHOWWINDOW;
	}
	else if (InParams.bLaunchMinimized)
	{
		dwFlags = STARTF_USESHOWWINDOW;
		ShowWindowFlags = SW_SHOWMINNOACTIVE;
	}

	if (InParams.StdoutPipe != nullptr || InParams.StdinPipe != nullptr || InParams.StderrPipe != nullptr)
	{
		dwFlags |= STARTF_USESTDHANDLES;
	}

	bool bInheritHandles = (dwFlags & STARTF_USESTDHANDLES) != 0;
	bInheritHandles |= InParams.bInheritAllHandles;
	bInheritHandles |= InParams.AdditionalInheritedHandles.Num() > 0;

	// initialize startup info
	CreateFlags |= EXTENDED_STARTUPINFO_PRESENT;
	STARTUPINFOEXW StartupInfoEx;
	FMemory::Memzero(StartupInfoEx);
	StartupInfoEx.StartupInfo = {
		sizeof(STARTUPINFOEXW),
		nullptr,
		nullptr,
		nullptr,
		(DWORD)CW_USEDEFAULT,
		(DWORD)CW_USEDEFAULT,
		(DWORD)CW_USEDEFAULT,
		(DWORD)CW_USEDEFAULT,
		(DWORD)0,
		(DWORD)0,
		(DWORD)0,
		(DWORD)dwFlags,
		ShowWindowFlags,
		0,
		nullptr,
		HANDLE(InParams.StdinPipe),
		HANDLE(InParams.StdoutPipe),
		HANDLE(InParams.StderrPipe)
	};

	ON_SCOPE_EXIT
	{
		if (StartupInfoEx.lpAttributeList)
		{
			DeleteProcThreadAttributeList(StartupInfoEx.lpAttributeList);
			FMemory::Free(StartupInfoEx.lpAttributeList);
			StartupInfoEx.lpAttributeList = nullptr;
		}
	};

	TArray<HANDLE> InheritableHandles;

	if (bInheritHandles && !InParams.bInheritAllHandles)
	{
		InheritableHandles.Reserve(InParams.AdditionalInheritedHandles.Num() + 3);

		InheritableHandles.Append(InParams.AdditionalInheritedHandles);

		if (InParams.StdinPipe)
		{
			InheritableHandles.Add(InParams.StdinPipe);
		}

		if (InParams.StdoutPipe)
		{
			InheritableHandles.Add(InParams.StdoutPipe);
		}

		if (InParams.StderrPipe && InParams.StderrPipe != InParams.StdoutPipe)
		{
			InheritableHandles.Add(InParams.StderrPipe);
		}

		// This "fails" with ERROR_INSUFFICIENT_BUFFER by design.
		const int32 NumAttributes = 1;
		SIZE_T AttrListSize = 0;
		InitializeProcThreadAttributeList(nullptr, NumAttributes, 0, &AttrListSize);

		const DWORD ErrorCode = GetLastError();
		if (ErrorCode != ERROR_INSUFFICIENT_BUFFER || AttrListSize <= 0)
		{
			UE::SwitchboardListener::Private::LogLastError(TEXT("InitializeProcThreadAttributeList"));
			return {};
		}

		StartupInfoEx.lpAttributeList = (LPPROC_THREAD_ATTRIBUTE_LIST)FMemory::Malloc(AttrListSize);

		if (!InitializeProcThreadAttributeList(StartupInfoEx.lpAttributeList, NumAttributes, 0, &AttrListSize))
		{
			UE::SwitchboardListener::Private::LogLastError(TEXT("InitializeProcThreadAttributeList"));
			return {};
		}

		if (!UpdateProcThreadAttribute(StartupInfoEx.lpAttributeList, 0, PROC_THREAD_ATTRIBUTE_HANDLE_LIST,
				InheritableHandles.GetData(), InheritableHandles.Num() * InheritableHandles.GetTypeSize(),
				nullptr, nullptr)
			)
		{
			UE::SwitchboardListener::Private::LogLastError(TEXT("UpdateProcThreadAttribute"));
			return {};
		}
	}

	// create the child process
	const FString Args = FString::Join(InParams.Args, TEXT(" "));
	FString CommandLine = FString::Printf(TEXT("\"%.*s\" %s"), InParams.Executable.Len(), InParams.Executable.GetData(), *Args);
	PROCESS_INFORMATION ProcInfo;

	if (!CreateProcess(
			NULL,
			CommandLine.GetCharArray().GetData(),
			nullptr,
			nullptr,
			bInheritHandles,
			(DWORD)CreateFlags,
			NULL,
			InParams.WorkingDirectory ? *FString(*InParams.WorkingDirectory) : nullptr,
			(LPSTARTUPINFOW)&StartupInfoEx,
			&ProcInfo
		))
	{
		const DWORD ErrorCode = GetLastError();
		UE::SwitchboardListener::Private::LogLastError(TEXT("CreateProcess"));
		if (ErrorCode == ERROR_NOT_ENOUGH_MEMORY || ErrorCode == ERROR_OUTOFMEMORY)
		{
			// These errors are common enough that we want some available memory information
			FPlatformMemoryStats Stats = FPlatformMemory::GetStats();
			UE_LOGF(LogWindows, Warning, "Mem used: %.2f MB, OS Free %.2f MB", Stats.UsedPhysical / 1048576.0f, Stats.AvailablePhysical / 1048576.0f);
		}
		UE_LOGF(LogWindows, Warning, "URL: %ls", *CommandLine);

		return {};
	}

	::CloseHandle( ProcInfo.hThread );

	return { FProcHandle(ProcInfo.hProcess), ProcInfo.dwProcessId };
}

#include "Windows/HideWindowsPlatformTypes.h"
