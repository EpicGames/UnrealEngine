// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class SblCore : ModuleRules
{
	public SblCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.AddRange(
			new string[]
			{
				"OpenSSL",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore", // for LaunchEngineLoop.cpp dependency
				"Projects", // for LaunchEngineLoop.cpp dependency
				"Core",
				"CoreUObject",
				"Networking",
				"MsQuic",
				"SwitchboardCommon",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Json",
				"JsonUtilities",
				"JWT",
				"MsQuicRuntime",
				"TraceLog",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Launch",
			}
		);

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"NVAPI",
				}
			);

			PublicSystemLibraries.Add("Pdh.lib");
		}

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"Sockets",
			}
		);

		AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PublicAdditionalLibraries.Add(Path.Combine(ModuleDirectory, "NvmlWrapper", "Lib", "Windows", "NvmlWrapper.lib"));
		}
		else if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			PublicAdditionalLibraries.Add(Path.Combine(ModuleDirectory, "NvmlWrapper", "Lib", "Linux", "libNvmlWrapper.a"));
		}

		PublicIncludePaths.AddRange(new string[] 
		{
			Path.Combine(ModuleDirectory, "NvmlWrapper", "Public"),
		});

	}
}
