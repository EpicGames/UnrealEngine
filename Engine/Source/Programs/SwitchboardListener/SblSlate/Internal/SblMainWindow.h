// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "SwitchboardListenerAutolaunch.h"
#include "Templates/SharedPointer.h"


enum class ECheckBoxState : uint8;
class FSwitchboardListener;
class SEditableTextBox;
class SWidget;
class SWidgetSwitcher;
class SWindow;
class UToolMenu;


class FSwitchboardListenerMainWindow
{
	enum class EPanelIndices
	{
		MainPanel = 0,
		PasswordPanel,
	};

public:
	FSwitchboardListenerMainWindow(FSwitchboardListener& InListener);

private:
	void OnInit();
	void OnShutdown();
	void OnTick();

private:
	void CustomizeToolMenus();
	void CustomizeToolMenus_AddGeneralSection(UToolMenu* InMenu);
	void CustomizeToolMenus_AddPasswordSection(UToolMenu* InMenu);
#if UE_BUILD_DEBUG
	void CustomizeToolMenus_AddDevelopmentSection(UToolMenu* InMenu);
#endif

	TSharedRef<SWidget> CreateRootSwitcher();
	TSharedRef<SWidget> CreateSetPasswordPanel();
	TSharedRef<SWidget> CreateMainPanel();
	TSharedRef<SWidget> CreateOutputLog();

	FReply OnSetPasswordClicked();

private:
	FSwitchboardListener& Listener;

	TSharedPtr<SWindow> RootWindow;
	TSharedPtr<SWidgetSwitcher> PanelSwitcher;

	TSharedPtr<SEditableTextBox> PasswordTextBox;

	bool bPasswordRevealedInEditBox = false;
	bool bPasswordRevealedInMenu = false;

#if SWITCHBOARD_LISTENER_AUTOLAUNCH
	ECheckBoxState CachedAutolaunchEnabled;
#endif
};
