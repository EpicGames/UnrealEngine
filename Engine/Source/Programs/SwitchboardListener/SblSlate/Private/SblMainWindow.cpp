// Copyright Epic Games, Inc. All Rights Reserved.

#include "SblMainWindow.h"
#include "SwitchboardListener.h"
#include "SwitchboardListenerApp.h"
#include "SwitchboardListenerVersion.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SWindowTitleBar.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ISlateReflectorModule.h"
#include "Logging/StructuredLog.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/EngineVersion.h"
#include "Modules/ModuleManager.h"
#include "OutputLogCreationParams.h"
#include "OutputLogModule.h"
#include "StandaloneRenderer.h"
#include "Styling/StyleColors.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWidgetSwitcher.h"


#define LOCTEXT_NAMESPACE "SwitchboardListener"


class SSblWindow : public SWindow
{
public:
	void Construct(const FArguments& InArgs)
	{
		FDisplayMetrics DisplayMetrics;
		FSlateApplication::Get().GetDisplayMetrics(DisplayMetrics);
		const float DPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(DisplayMetrics.PrimaryDisplayWorkAreaRect.Left, DisplayMetrics.PrimaryDisplayWorkAreaRect.Top);

		const FString EngineVersionTitleString = FEngineVersion::Current().ToString(EVersionComponent::Patch);
		const FText WindowTitle = FText::Format(
			LOCTEXT("WindowTitle", "Switchboard Listener (Unreal Engine {0})"),
			FText::FromString(EngineVersionTitleString));

		SWindow::Construct(SWindow::FArguments()
			.Title(WindowTitle)
			.CreateTitleBar(true)
			.SupportsMaximize(true)
			.SupportsMinimize(true)
			.IsInitiallyMaximized(false)
			.IsInitiallyMinimized(false)
			.SizingRule(ESizingRule::UserSized)
			.AutoCenter(EAutoCenter::PreferredWorkArea)
			.ClientSize(FVector2D(1000.0f * DPIScaleFactor, 500.0f * DPIScaleFactor))
			.AdjustInitialSizeAndPositionForDPIScale(false)
		);

		// Create the tooltip showing more detailed information
		const FString AppVersionString = FString::Printf(TEXT("%u.%u.%u"), SBLISTENER_VERSION_MAJOR, SBLISTENER_VERSION_MINOR, SBLISTENER_VERSION_PATCH);
		const FString EngineVersionTooltipString = FEngineVersion::Current().ToString(FEngineVersion::Current().HasChangelist() ? EVersionComponent::Changelist : EVersionComponent::Patch);
		const EBuildConfiguration BuildConfig = FApp::GetBuildConfiguration();

		FFormatNamedArguments TooltipArgs;
		TooltipArgs.Add(TEXT("AppVersion"), FText::FromString(AppVersionString));
		TooltipArgs.Add(TEXT("EngineVersion"), FText::FromString(EngineVersionTooltipString));
		TooltipArgs.Add(TEXT("Branch"), FText::FromString(FEngineVersion::Current().GetBranch()));
		TooltipArgs.Add(TEXT("BuildConfiguration"), EBuildConfigurations::ToText(BuildConfig));
		TooltipArgs.Add(TEXT("BuildDate"), FText::FromString(FApp::GetBuildDate()));

		const FText TitleToolTip = FText::Format(
			LOCTEXT("TitleBarTooltip", "Switchboard Listener Version: {AppVersion}\nEngine Version: {EngineVersion}\nBranch: {Branch}\nBuild Configuration: {BuildConfiguration}\nBuild Date: {BuildDate}"), TooltipArgs);

		AddOverlaySlot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Margin(FMargin(0.0, 12.0, 0.0, 0.0))
			.Text(WindowTitle)
			.ToolTipText(TitleToolTip)
			.ColorAndOpacity(FLinearColor::Transparent)
		];
	}
};


FSwitchboardListenerMainWindow::FSwitchboardListenerMainWindow(FSwitchboardListener& InListener)
	: Listener(InListener)
#if SWITCHBOARD_LISTENER_AUTOLAUNCH
	, CachedAutolaunchEnabled(ECheckBoxState::Unchecked)
#endif
{
	Listener.OnInit().AddRaw(this, &FSwitchboardListenerMainWindow::OnInit);
	Listener.OnShutdown().AddRaw(this, &FSwitchboardListenerMainWindow::OnShutdown);
	Listener.OnTick().AddRaw(this, &FSwitchboardListenerMainWindow::OnTick);
}


void FSwitchboardListenerMainWindow::OnInit()
{
	FSlateApplication::InitializeAsStandaloneApplication(GetStandardStandaloneRenderer());
	FSlateApplication::InitHighDPI(true);

	FModuleManager::Get().LoadModuleChecked("SlateCore");
	FModuleManager::Get().LoadModuleChecked("OutputLog");

	if (IConsoleVariable* HideConsoleCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("OutputLogModule.HideConsole")); ensure(HideConsoleCVar))
	{
		HideConsoleCVar->Set(true);
	}

	CustomizeToolMenus();

	const FText ApplicationTitle = LOCTEXT("AppTitle", "Switchboard Listener");
	FGlobalTabmanager::Get()->SetApplicationTitle(ApplicationTitle);

	RootWindow = SNew(SSblWindow);

	constexpr bool bShowRootWindowImmediately = true;
	FSlateApplication::Get().AddWindow(RootWindow.ToSharedRef(), bShowRootWindowImmediately);
	FGlobalTabmanager::Get()->SetRootWindow(RootWindow.ToSharedRef());
	FGlobalTabmanager::Get()->SetAllowWindowMenuBar(true);
	FSlateNotificationManager::Get().SetRootWindow(RootWindow.ToSharedRef());

	RootWindow->SetContent(CreateRootSwitcher());

	if (!Listener.IsAuthPasswordSet())
	{
		PanelSwitcher->SetActiveWidgetIndex(static_cast<int32>(EPanelIndices::PasswordPanel));
	}
	else
	{
		if (ensure(Listener.StartListening()))
		{
			UE_LOGFMT(LogSwitchboard, Display, "Started listening");
		}
		else
		{
			UE_LOGFMT(LogSwitchboard, Error, "StartListening failed");
		}
	}

	RootWindow->ShowWindow();
	constexpr bool bForceWindowToFront = true;
	RootWindow->BringToFront(bForceWindowToFront);
}


void FSwitchboardListenerMainWindow::OnTick()
{
	FSlateApplication::Get().PumpMessages();
	FSlateApplication::Get().Tick();
}


void FSwitchboardListenerMainWindow::OnShutdown()
{
	RootWindow.Reset();
	FSlateApplication::Shutdown();
}


void FSwitchboardListenerMainWindow::CustomizeToolMenus()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	check(ToolMenus);

	static const FName MenuName = "OutputLog.SettingsMenu";
	UToolMenu* Menu = ToolMenus->ExtendMenu(MenuName);

	FCustomizedToolMenu* CustomizedMenu = ToolMenus->AddRuntimeMenuCustomization(MenuName);
	CustomizedMenu->AddEntry("Separator")->Visibility = ECustomizedToolMenuVisibility::Hidden;

	CustomizeToolMenus_AddGeneralSection(Menu);
	CustomizeToolMenus_AddPasswordSection(Menu);
#if UE_BUILD_DEBUG
	CustomizeToolMenus_AddDevelopmentSection(Menu);
#endif

	if (FToolMenuSection* DefaultSection = Menu->FindSection(NAME_None); ensure(DefaultSection))
	{
		DefaultSection->Label = LOCTEXT("SettingsMenu_OutputLogSection_Label", "Output Log");

		if (FToolMenuEntry* BrowseLogsEntry = DefaultSection->FindEntry("BrowseLogDirectory"); ensure(BrowseLogsEntry))
		{
			BrowseLogsEntry->Label = LOCTEXT("SettingsMenu_BrowseLogDirectory_Label", "Open Logs Folder");
		}
	}
}


void FSwitchboardListenerMainWindow::CustomizeToolMenus_AddGeneralSection(UToolMenu* InMenu)
{
	FToolMenuSection& GeneralSection = InMenu->AddSection("SBL_General", LOCTEXT("SettingsMenu_GeneralSection_Label", "General"), FToolMenuInsert(NAME_None, EToolMenuInsertType::First));

#if SWITCHBOARD_LISTENER_AUTOLAUNCH
	GeneralSection.AddDynamicEntry("LaunchOnLoginDynamic", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
	{
		const FString AutolaunchExe = UE::SwitchboardListener::Autolaunch::GetInvocationExecutable(LogSwitchboard);
		const FString CurrentExe = FPlatformProcess::ExecutablePath();
		if (AutolaunchExe.IsEmpty())
		{
			CachedAutolaunchEnabled = ECheckBoxState::Unchecked;
		}
		else if (AutolaunchExe == CurrentExe)
		{
			CachedAutolaunchEnabled = ECheckBoxState::Checked;
		}
		else
		{
			CachedAutolaunchEnabled = ECheckBoxState::Undetermined;
		}

		FToolUIAction LaunchOnLoginAction;
		LaunchOnLoginAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([this](const FToolMenuContext&)
			{
				return CachedAutolaunchEnabled;
			});
		LaunchOnLoginAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([this, CurrentExe](const FToolMenuContext&)
			{
				if (CachedAutolaunchEnabled == ECheckBoxState::Checked)
				{
					if (ensure(UE::SwitchboardListener::Autolaunch::RemoveInvocation(LogSwitchboard)))
					{
						CachedAutolaunchEnabled = ECheckBoxState::Unchecked;
					}
				}
				else
				{
					if (ensure(UE::SwitchboardListener::Autolaunch::SetInvocation(FString::Printf(TEXT("\"%s\""), *CurrentExe), LogSwitchboard)))
					{
						CachedAutolaunchEnabled = ECheckBoxState::Checked;
					}
				}
			});

		InSection.AddMenuEntry(
			"LaunchOnLogin",
			LOCTEXT("SettingsMenu_LaunchOnLogin_Label", "Launch Switchboard Listener on Login"),
			FText(),
			FSlateIcon(),
			LaunchOnLoginAction,
			EUserInterfaceActionType::ToggleButton
		);
	}));
#endif // #if SWITCHBOARD_LISTENER_AUTOLAUNCH
}


void FSwitchboardListenerMainWindow::CustomizeToolMenus_AddPasswordSection(UToolMenu* InMenu)
{
	FToolMenuSection& PasswordSection = InMenu->AddSection("SBL_Password", LOCTEXT("SettingsMenu_PasswordSection_Label", "Password"), FToolMenuInsert("SBL_General", EToolMenuInsertType::After));

	const FText& PasswordEntryLabel = Listener.GetAuthPassword() != nullptr
		? LOCTEXT("SettingsMenu_PasswordPanel_CanShow_Label", "Show/Change Password")
		: LOCTEXT("SettingsMenu_PasswordPanel_NoShow_Label", "Change Password");

	PasswordSection.AddMenuEntry("PasswordPanel", PasswordEntryLabel, FText(), FSlateIcon(),
		FExecuteAction::CreateLambda([this]()
			{
				PasswordTextBox->SetText(FText::FromString(Listener.GetAuthPassword()));
				PanelSwitcher->SetActiveWidgetIndex(static_cast<int32>(EPanelIndices::PasswordPanel));
			})
	);
}


#if UE_BUILD_DEBUG
void FSwitchboardListenerMainWindow::CustomizeToolMenus_AddDevelopmentSection(UToolMenu* InMenu)
{
	FToolMenuSection& DevelopmentSection = InMenu->AddSection("SBL_Development", LOCTEXT("SettingsMenu_DevelopmentSection_Label", "Development"), FToolMenuInsert(NAME_None, EToolMenuInsertType::After));
	DevelopmentSection.AddMenuEntry("WidgetReflector", LOCTEXT("SettingsMenu_WidgetReflector_Label", "Widget Reflector"), FText(), FSlateIcon(),
		FExecuteAction::CreateStatic([]()
			{
				FModuleManager::LoadModuleChecked<ISlateReflectorModule>("SlateReflector").DisplayWidgetReflector();
			})
	);
}
#endif // #if UE_BUILD_DEBUG


TSharedRef<SWidget> FSwitchboardListenerMainWindow::CreateRootSwitcher()
{
	ensure(!PanelSwitcher.IsValid());
	PanelSwitcher = SNew(SWidgetSwitcher);

	PanelSwitcher->AddSlot(static_cast<int32>(EPanelIndices::MainPanel))
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Brushes.Panel"))
				.Visibility(EVisibility::HitTestInvisible)
			]
			+ SOverlay::Slot()
			[
				CreateMainPanel()
			]
		];

	PanelSwitcher->AddSlot(static_cast<int32>(EPanelIndices::PasswordPanel))
		[
			CreateSetPasswordPanel()
		];

	return PanelSwitcher.ToSharedRef();
}


TSharedRef<SWidget> FSwitchboardListenerMainWindow::CreateSetPasswordPanel()
{
	TSharedPtr<STextBlock> HeadingTextBlock, IntroTextBlock;

	TSharedRef<SWidget> SetPasswordPanel = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SNullWidget::NullWidget
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNullWidget::NullWidget
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(HeadingTextBlock, STextBlock)
				.Text(LOCTEXT("PasswordPanel_WelcomeHeading", "Welcome to Switchboard Listener"))
				.Font(FAppStyle::Get().GetFontStyle("HeadingMedium"))
				.TransformPolicy(ETextTransformPolicy::ToUpper)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 24.0f)
			[
				SAssignNew(IntroTextBlock, STextBlock)
				.Text(LOCTEXT("PasswordPanel_WelcomeIntro", "Please set a password for Switchboard Listener on this machine. This password will be used to authenticate and establish a secure connection with Switchboard."))
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 12))
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SAssignNew(PasswordTextBox, SEditableTextBox)
					.IsPassword_Lambda([this]() { return !bPasswordRevealedInEditBox; })
					.HintText(LOCTEXT("PasswordPanel_Edit_Hint", "Password"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(3.0f)
				[
					SNew(SSpacer)
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
					.IsChecked_Lambda([this]() { return bPasswordRevealedInEditBox ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					.OnCheckStateChanged_Lambda([this](ECheckBoxState InNewState)
						{
							bPasswordRevealedInEditBox = (InNewState == ECheckBoxState::Checked);
						})
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PasswordPanel_ShowPasswordCheckLabel", "Show Password"))
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SButton)
					.Text_Lambda([this]()
						{
							return Listener.IsAuthPasswordSet()
								? LOCTEXT("PasswordPanel_ChangePasswordButton", "Change Password")
								: LOCTEXT("PasswordPanel_SetPasswordButton", "Set Password");
						})
					.IsEnabled_Lambda([this]() { return !PasswordTextBox->GetText().IsEmpty(); })
					.OnClicked_Raw(this, &FSwitchboardListenerMainWindow::OnSetPasswordClicked)
					.ButtonStyle(&FAppStyle::Get(), "PrimaryButton")
					.TextStyle(&FAppStyle::Get(), "PrimaryButtonText")
					.ContentPadding(FMargin(4.0, 2.0 + 2.0))
					.HAlign(HAlign_Center)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(4.0f, 0.0f)
				[
					SNew(SButton)
					.Text(LOCTEXT("PasswordPanel_CancelButton", "Cancel"))
					.Visibility_Lambda([this]() { return Listener.IsAuthPasswordSet() ? EVisibility::Visible : EVisibility::Collapsed; })
					.OnClicked_Lambda([this]()
						{
							PanelSwitcher->SetActiveWidgetIndex(static_cast<int32>(EPanelIndices::MainPanel));
							return FReply::Handled();
						})
					.ContentPadding(FMargin(4.0, 2.0 + 2.0))
					.HAlign(HAlign_Center)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(2.0f)
				[
					SNew(SSpacer)
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SSpacer)
			]
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SNew(SSpacer)
		];

	// Constrain the body paragraph to the width of the heading.
	const float LayoutScaling = FSlateApplication::Get().GetApplicationScale() * RootWindow->GetNativeWindow()->GetDPIScaleFactor();
	const float HeadingWidth = HeadingTextBlock->ComputeDesiredSize(LayoutScaling).X;
	IntroTextBlock->SetWrapTextAt(HeadingWidth);

	return SetPasswordPanel;
}


TSharedRef<SWidget> FSwitchboardListenerMainWindow::CreateMainPanel()
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0)
		[
			CreateOutputLog()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.0f, 4.0f, 0.0f, 4.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ConnectionStatusIndicator", "Status"))
				.Font(FCoreStyle::GetDefaultFontStyle("Medium", 12))
				.ColorAndOpacity_Lambda([this]()
					{
						switch (Listener.GetConnectedClientAddresses().Num())
						{
							case 0: return FStyleColors::AccentRed;
							case 1: return FStyleColors::AccentGreen;
							default: return FStyleColors::AccentOrange;
						}
					})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
					{
						const TArray<FIPv4Address> ConnectedAddrs = Listener.GetConnectedClientAddresses().Array();
						if (ConnectedAddrs.Num() <= 0)
						{
							return LOCTEXT("ConnectionStatusText_NotConnected", "Not Connected");
						}
						else if (ConnectedAddrs.Num() == 1)
						{
							return FText::Format(
								LOCTEXT("ConnectionStatusText_Connected",
								        "Connected to {0}"),
								FText::FromString(ConnectedAddrs[0].ToString()));
						}
						else
						{
							return LOCTEXT("ConnectionStatusText_ConnectedMulti", "Connected to multiple clients");
						}
					})
				.ToolTipText_Lambda([this]()
					{
						const TArray<FIPv4Address> ConnectedAddrs = Listener.GetConnectedClientAddresses().Array();
						if (ConnectedAddrs.Num() > 1)
						{
							FFormatOrderedArguments TextAddrs;
							TextAddrs.Reserve(ConnectedAddrs.Num());
							for (const FIPv4Address& Addr : ConnectedAddrs)
							{
								TextAddrs.Emplace(Addr.ToText());
							}
							const FText ConnectedAddrListText = FText::Join(LOCTEXT("ConnectionStatusTooltip_AddrDelimiter", ", "), TextAddrs);
							return FText::Format(LOCTEXT("ConnectionStatusTooltip_ConnectedMulti", "Connected to multiple clients: {0}"),
								ConnectedAddrListText);
						}

						return FText::GetEmpty();
					})
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0)
			[
				SNew(SSpacer)
			]
		]
	;
}


TSharedRef<SWidget> FSwitchboardListenerMainWindow::CreateOutputLog()
{
	FOutputLogModule& OutputLogModule = FModuleManager::Get().LoadModuleChecked<FOutputLogModule>("OutputLog");

	FOutputLogCreationParams OutputLogCreationParams;

	OutputLogCreationParams.AllowAsInitialLogCategory = FAllowLogCategoryCallback::CreateLambda([](const FName LogCategory) {
		return LogCategory == LogSwitchboard.GetCategoryName();
	});

	OutputLogCreationParams.DefaultCategorySelection.Emplace(LogSwitchboard.GetCategoryName(), true);
	OutputLogCreationParams.DefaultCategorySelection.Emplace(LogSwitchboardProtocol.GetCategoryName(), false);

	// Limit logging output to 10,000 so that we don't over consume on memory.
	OutputLogCreationParams.LoggingLimit = 10000;
	TSharedRef<SWidget> OutputLog = OutputLogModule.MakeOutputLogWidget(OutputLogCreationParams);
	OutputLogModule.SuspendGlobalLog();
	return OutputLog;

}


FReply FSwitchboardListenerMainWindow::OnSetPasswordClicked()
{
	const FString EnteredPassword = PasswordTextBox->GetText().ToString();
	if (ensure(!EnteredPassword.IsEmpty()))
	{
		Listener.SetAuthPassword(EnteredPassword);
		if (!Listener.IsListening())
		{
			if (ensure(Listener.StartListening()))
			{
				UE_LOGFMT(LogSwitchboard, Display, "Started listening");
			}
			else
			{
				UE_LOGFMT(LogSwitchboard, Error, "StartListening failed");
			}
		}

		PanelSwitcher->SetActiveWidgetIndex(static_cast<int32>(EPanelIndices::MainPanel));
	}

	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
