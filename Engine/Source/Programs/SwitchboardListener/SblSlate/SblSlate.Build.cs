// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;


public class SblSlate : ModuleRules
{
	public SblSlate(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDefinitions.Add("SBL_SLATE=1");

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"SblCore",
				"SlateCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"OutputLog",
				"Slate",
				"SlateReflector",
				"StandaloneRenderer",
				"ToolMenus",
			}
		);
	}
}
