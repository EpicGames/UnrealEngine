// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;


public class SwitchboardListenerTargetBase : TargetRules
{
	public SwitchboardListenerTargetBase(TargetInfo Target) : base(Target)
	{
		SolutionDirectory = "Programs/Switchboard";

		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;

		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

		// This app compiles against Core/CoreUObject, but not the Engine or Editor, so compile out Engine and Editor references from Core/CoreUObject
		bCompileAgainstCoreUObject = true;
		bCompileAgainstEngine = false;
		bBuildWithEditorOnlyData = false;

		// Disable internationalization support, so the exe can be launched outside of the engine directory.
		bCompileICU = false;

		bEnableTrace = true;
	}
}


public class SwitchboardListenerTarget : SwitchboardListenerTargetBase
{
	public SwitchboardListenerTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "SwitchboardListener";
	}
}
