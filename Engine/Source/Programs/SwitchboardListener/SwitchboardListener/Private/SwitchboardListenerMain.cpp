// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardListenerMainCommon.h"
#include "SwitchboardListener.h"
#include "SwitchboardListenerVersion.h"
#include "SblMainWindow.h"

#include "RequiredProgramMainCPPInclude.h"
#include "Stats/StatsSystem.h"


IMPLEMENT_APPLICATION(SwitchboardListener, "SwitchboardListener");


int32 SwitchboardListenerMain()
{
	const TCHAR* CommandLine = FCommandLine::Get();
	const FSwitchboardCommandLineOptions Options = FSwitchboardCommandLineOptions::FromString(CommandLine);

	if (Options.OutputVersion)
	{
		printf("SwitchboardListener %u.%u.%u", SBLISTENER_VERSION_MAJOR, SBLISTENER_VERSION_MINOR, SBLISTENER_VERSION_PATCH);
		RequestEngineExit(TEXT("Output -version"));
		return 0;
	}

	if (!SwitchboardListenerMainInit(Options))
	{
		return 1;
	}

	FSwitchboardListener Listener(Options);
	FSwitchboardListenerMainWindow MainWindow(Listener);

	if (!Listener.Init())
	{
		RequestEngineExit(TEXT("FSwitchboardListener::Init() failure"));
		return 1;
	}

	UE_LOGF(LogSwitchboard, Display, "Initialized");

	double LastTime = FPlatformTime::Seconds();
	const float IdealFrameTime = 1.0f / 30.0f;

	while (!IsEngineExitRequested())
	{
		BeginExitIfRequested();

		const double CurrentTime = FPlatformTime::Seconds();
		const double DeltaTime = CurrentTime - LastTime;
		LastTime = CurrentTime;

		const double HitchThresholdSec = 0.5;
		if (DeltaTime > HitchThresholdSec)
		{
			UE_LOGF(LogSwitchboard, Warning, "Hitch detected; %.3f seconds since prior tick", DeltaTime);
		}

		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

		// Pump & Tick objects
		FTSTicker::GetCoreTicker().Tick(DeltaTime);

		Listener.Tick();

		GFrameCounter++;
		UE::Stats::FStats::AdvanceFrame(false);
		GLog->FlushThreadedLogs();

		// Run garbage collection for the UObjects for the rest of the frame or at least to 2 ms
		IncrementalPurgeGarbage(true, FMath::Max<float>(0.002f, IdealFrameTime - (FPlatformTime::Seconds() - CurrentTime)));

		// Throttle main thread main fps by sleeping if we still have time
		FPlatformProcess::Sleep(FMath::Max<float>(0.0f, IdealFrameTime - (FPlatformTime::Seconds() - CurrentTime)));
	}

	Listener.Shutdown();

	SwitchboardListenerMainShutdown();

	return 0;
}


#if PLATFORM_WINDOWS
int WINAPI WinMain(_In_ HINSTANCE hInInstance, _In_opt_ HINSTANCE hPrevInstance, _In_ LPSTR lpCmdLine, _In_ int nShowCmd)
{
	hInstance = hInInstance;

	const TCHAR* CommandLine = ::GetCommandLineW();
	CommandLine = FCommandLine::RemoveExeName(CommandLine);
	FCommandLine::Set(CommandLine);

	return SwitchboardListenerMainWrapper();
}
#else
INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	const FString CommandLine = FCommandLine::BuildFromArgV(nullptr, ArgC, ArgV, nullptr);
	FCommandLine::Set(*CommandLine);

	return SwitchboardListenerMainWrapper();
}
#endif
