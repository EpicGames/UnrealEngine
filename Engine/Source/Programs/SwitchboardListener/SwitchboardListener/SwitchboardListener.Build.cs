// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;


public class SwitchboardListener : SwitchboardListenerCmd
{
	public SwitchboardListener(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"SblSlate",
			}
		);
	}
}
