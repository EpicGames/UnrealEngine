// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;


public class SwitchboardListenerCmdTarget : SwitchboardListenerTargetBase
{
	public SwitchboardListenerCmdTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "SwitchboardListenerCmd";

		// The listener is meant to be a console application (no window), but
		// on MacOS, to get a proper log console, a full application must be built.
		bIsBuildingConsoleApplication = Target.Platform != UnrealTargetPlatform.Mac;
	}
}
