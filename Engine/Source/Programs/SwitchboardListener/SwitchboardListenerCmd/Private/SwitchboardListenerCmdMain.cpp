// Copyright Epic Games, Inc. All Rights Reserved.

#include "SwitchboardListenerMainCommon.h"
#include "SwitchboardListener.h"
#include "SwitchboardAuth.h"
#include "SwitchboardListenerVersion.h"

#include "RequiredProgramMainCPPInclude.h"
#include "Stats/StatsSystem.h"


IMPLEMENT_APPLICATION(SwitchboardListenerCmd, "SwitchboardListenerCmd");


int32 SwitchboardListenerMain()
{
	const TCHAR* CommandLine = FCommandLine::Get();
	const FSwitchboardCommandLineOptions Options = FSwitchboardCommandLineOptions::FromString(CommandLine);

	if (Options.OutputVersion)
	{
		printf("SwitchboardListener %u.%u.%u", SBLISTENER_VERSION_MAJOR, SBLISTENER_VERSION_MINOR, SBLISTENER_VERSION_PATCH);
		RequestEngineExit(TEXT("Output -version"));
		return 0;
	}

#if PLATFORM_WINDOWS
	// If we were launched detached, detect and spawn a new console.
	// TODO: Command line switch to suppress this behavior? Or explicitly opt-in for redeploy?
	{
		bool bIsStdoutAttachedToConsole = false;
		HANDLE StdoutHandle = GetStdHandle(STD_OUTPUT_HANDLE);
		if (StdoutHandle != INVALID_HANDLE_VALUE)
		{
			if (GetFileType(StdoutHandle) == FILE_TYPE_CHAR)
			{
				bIsStdoutAttachedToConsole = true;
			}
		}

		if (!bIsStdoutAttachedToConsole)
		{
			AllocConsole();

			FILE* OutIgnoredNewStreamPtr = nullptr;
			freopen_s(&OutIgnoredNewStreamPtr, "CONIN$", "r", stdin);
			freopen_s(&OutIgnoredNewStreamPtr, "CONOUT$", "w", stdout);
			freopen_s(&OutIgnoredNewStreamPtr, "CONOUT$", "w", stderr);
		}
	}

	// Sets the window title. It informs the version of the listener and the UE version it is based on.
	const FString Title = FString::Printf(TEXT("Switchboard Listener %d.%d.%d based on Unreal Engine %s"),
		SBLISTENER_VERSION_MAJOR, SBLISTENER_VERSION_MINOR, SBLISTENER_VERSION_PATCH, *FEngineVersion::Current().ToString());

	::SetConsoleTitle(*Title);
#endif

	if (!SwitchboardListenerMainInit(Options))
	{
		return 1;
	}

	FSwitchboardListener Listener(Options);
	Listener.Init();

	if (!Listener.IsAuthPasswordSet())
	{
		GLog->Flush();

		printf(
			"\nPlease set a password for Switchboard Listener on this machine.\n"
			"This password will be used to authenticate and establish a secure connection with Switchboard.\n");

		Listener.SetAuthPassword(UE::SwitchboardListener::ReadPasswordFromStdin());
	}

	if (!Listener.StartListening())
	{
		RequestEngineExit(TEXT("FSwitchboardListener::StartListening() failure"));
		return 1;
	}

#if PLATFORM_WINDOWS && !UE_BUILD_DEBUG
	if (Options.MinimizeOnLaunch)
	{
		::ShowWindow(::GetConsoleWindow(), SW_MINIMIZE);
	}
#endif

	UE_LOGF(LogSwitchboard, Display, "Initialized and listening");

	double LastTime = FPlatformTime::Seconds();
	const float IdealFrameTime = 1.0f / 30.0f;

	while (!IsEngineExitRequested())
	{
		BeginExitIfRequested();

		const double CurrentTime = FPlatformTime::Seconds();
		const double DeltaTime = CurrentTime - LastTime;
		LastTime = CurrentTime;

		const double HitchThresholdSec = 0.5;
		if (DeltaTime > HitchThresholdSec)
		{
			UE_LOGF(LogSwitchboard, Warning, "Hitch detected; %.3f seconds since prior tick", DeltaTime);
		}

		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

		// Pump & Tick objects
		FTSTicker::GetCoreTicker().Tick(DeltaTime);

		Listener.Tick();

		GFrameCounter++;
		UE::Stats::FStats::AdvanceFrame(false);
		GLog->FlushThreadedLogs();

		// Run garbage collection for the UObjects for the rest of the frame or at least to 2 ms
		IncrementalPurgeGarbage(true, FMath::Max<float>(0.002f, IdealFrameTime - (FPlatformTime::Seconds() - CurrentTime)));

		// Throttle main thread main fps by sleeping if we still have time
		FPlatformProcess::Sleep(FMath::Max<float>(0.0f, IdealFrameTime - (FPlatformTime::Seconds() - CurrentTime)));
	}

	Listener.Shutdown();

	SwitchboardListenerMainShutdown();

	return 0;
}


INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	const FString CommandLine = FCommandLine::BuildFromArgV(nullptr, ArgC, ArgV, nullptr);
	FCommandLine::Set(*CommandLine);

	return SwitchboardListenerMainWrapper();
}
