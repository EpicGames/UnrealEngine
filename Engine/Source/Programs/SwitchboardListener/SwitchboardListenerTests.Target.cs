// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;


[SupportedPlatforms("Win64", "Linux")]
public class SwitchboardListenerTestsTarget : TestTargetRules
{
	public SwitchboardListenerTestsTarget(TargetInfo Target) : base(Target)
	{
		SolutionDirectory = "Programs/Switchboard";
		LaunchModuleName = "SwitchboardListenerTests";

		// Don't pull in LaunchEngineLoop
		GlobalDefinitions.Add("SWITCHBOARD_LISTENER_EXCLUDE_MAIN=1");

		ExtraModuleNames.Add(LaunchModuleName);
	}
}
