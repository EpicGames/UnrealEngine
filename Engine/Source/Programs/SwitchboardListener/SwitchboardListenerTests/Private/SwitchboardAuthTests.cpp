// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestHarness.h"

#include "SwitchboardAuth.h"
#include "SwitchboardCredentialInterface.h"

#include "JsonWebToken.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "Tests/WarnFilterScope.h"


namespace UE::SwitchboardListener::Private::Tests
{
	class FMockCredentialManager : public ICredentialManager
	{
		virtual TSharedPtr<ICredential> LoadCredential(FStringView InCredentialName) override
		{
			// no-op
			return {};
		}

		virtual bool SaveCredential(FStringView InCredentialName, FStringView InUser, FStringView InBlob) override
		{
			// no-op
			return true;
		}
	};

	TEST_CASE("UE::SwitchboardListener::Auth")
	{
		EVP_PKEY* PrivateKey = nullptr;
		X509* Certificate = nullptr;
		Tie(PrivateKey, Certificate) = CreateSelfSignedPair();

		SECTION("Self-signed key pair generation")
		{
			REQUIRE(PrivateKey != nullptr);
			REQUIRE(Certificate != nullptr);
		}

		//////////////////////////////////////////////////////////////////////////

		FSwitchboardAuthHelper::FSettings Settings;
		Settings.Certificate.Emplace(TInPlaceType<X509*>(), Certificate);
		Settings.PrivateKey.Emplace(TInPlaceType<EVP_PKEY*>(), PrivateKey);

		FMockCredentialManager MockCredMgr;
		Settings.CredentialManager = &MockCredMgr;

		SECTION("Password hashing")
		{
			constexpr FStringView TestPassword = TEXTVIEW("ExamplePassword");
			constexpr FStringView IncorrectPassword = TEXTVIEW("WrongPassword");

			FSwitchboardAuthHelper AuthHelper;
			CHECK(AuthHelper.Initialize(Settings));

			CHECK(!AuthHelper.IsAuthPasswordSet());

			CHECK(AuthHelper.SetAuthPassword(TestPassword));

			CHECK(AuthHelper.IsAuthPasswordSet());

			CHECK(!AuthHelper.ValidatePassword(IncorrectPassword));

			CHECK(AuthHelper.ValidatePassword(TestPassword));
		}

		//////////////////////////////////////////////////////////////////////////

		SECTION("JWT")
		{
			FSwitchboardAuthHelper AuthHelper;
			CHECK(AuthHelper.Initialize(Settings));

			constexpr FStringView TestClaimName = TEXTVIEW("MyTestJwtField");
			constexpr FStringView TestClaimValue = TEXTVIEW("MyTestJwtValue");

			const FString JwtString = AuthHelper.IssueJWT({
				{ TestClaimName, TestClaimValue }
			});

			const double JwtIssuePlatformTime = FPlatformTime::Seconds();

			CHECK(!JwtString.IsEmpty());

			TOptional<FJsonWebToken> ValidatedJwt = AuthHelper.GetValidatedJWT(JwtString);
			CHECK(ValidatedJwt.IsSet());

			if (ValidatedJwt)
			{
				TSharedPtr<FJsonValue> ValidatedPayloadField =
					ValidatedJwt->GetClaim<EJson::String>(TestClaimName);
				CHECK(ValidatedPayloadField);
				if (ValidatedPayloadField)
				{
					FString ValidatedClaimValue;
					const bool bGotClaimString = ValidatedPayloadField->TryGetString(ValidatedClaimValue);
					CHECK(bGotClaimString);
					if (bGotClaimString)
					{
						CHECK_EQUAL(ValidatedClaimValue, TestClaimValue);
					}
				}
			}

			// Test expiry. Initially, the JWT is valid.
			CHECK(AuthHelper.IsValidJWT(JwtString));

			// We need at least one second to elapse after issuing the JWT.
			const double SecondsSinceIssuance = FPlatformTime::Seconds() - JwtIssuePlatformTime;
			FPlatformProcess::Sleep(1.0f - SecondsSinceIssuance);

			const bool bExpireExistingTokens_true = true;
			CHECK(AuthHelper.SetAuthPassword(TEXTVIEW("DifferentPassword"), bExpireExistingTokens_true));

			// Previously issued JWT was expired by a password change.
			{
				FWarnFilterScope _([](const TCHAR* Message, ELogVerbosity::Type Verbosity, const FName& Category)
					{
						constexpr FStringView ExpectedError = TEXTVIEW("JWT issued at time predates last password change");
						const bool bFiltered =
							0 == FCString::Strncmp(Message, ExpectedError.GetData(), ExpectedError.Len())
							&& Verbosity == ELogVerbosity::Type::Error
							&& Category == TEXT("LogSwitchboard");

						return bFiltered;
					});

				CHECK(!AuthHelper.IsValidJWT(JwtString));
			}
		}
	}
} // namespace UE::SwitchboardListener::Private::Tests
