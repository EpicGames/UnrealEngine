// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestHarness.h"

#include "SwitchboardAuth.h"
#include "SwitchboardCredential.h"

#include "HAL/PlatformTime.h"
#include "Math/UnrealMathUtility.h"


namespace UE::SwitchboardListener::Private::Tests
{
	TEST_CASE("UE::SwitchboardListener::Credential")
	{
		FMath::RandInit(FPlatformTime::Cycles());
		FMath::SRandInit(FPlatformTime::Cycles());

		SECTION("Basic save and load")
		{
			constexpr FStringView TestCredentialName = TEXTVIEW("TestCredential");
			const FString TestRandomUser = FString::FromInt(FMath::RandRange(int32(0), MAX_int32-1));
			const FString TestRandomBlob = FString::FromInt(FMath::RandRange(int32(0), MAX_int32-1));

			ICredentialManager& CredMgr = ICredentialManager::GetPlatformCredentialManager();
			CHECK(CredMgr.SaveCredential(TestCredentialName, TestRandomUser, TestRandomBlob));

			TSharedPtr<ICredential> LoadedCredential = CredMgr.LoadCredential(TestCredentialName);
			CHECK(LoadedCredential);
			if (LoadedCredential)
			{
				CHECK_EQUAL(LoadedCredential->GetUser(), TestRandomUser);
				CHECK_EQUAL(LoadedCredential->GetBlob(), TestRandomBlob);
			}
		}
	}
} // namespace UE::SwitchboardListener::Private::Tests
