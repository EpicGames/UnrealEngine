// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;


public class SwitchboardListenerTests : TestModuleRules
{
	static SwitchboardListenerTests()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "SwitchboardListener";
			TestMetadata.TestShortName = "SwitchboardListener";
		}
	}

	public SwitchboardListenerTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(new string[] {
			"SblCore",

			"JWT",
		});

		PrivateIncludePathModuleNames.Add("LowLevelTestsRunner");
	}
}
