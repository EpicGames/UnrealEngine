# Copyright Epic Games, Inc. All Rights Reserved.

import base64
import json
import os
import socket
import time
import unittest
import uuid

LISTENER_IP = "0.0.0.0" # needs to be set to the local ip the listener is listening on
LISTENER_PORT = 2980
TEST_PROGRAM_EXE = "C:\\Program Files\\Git\\bin\\git.exe"
TEST_FILE_TO_TRANSFER = "transfer_test.cfg"
SOURCE_CONTROL_CONNECTION = "perforce:1666"
SOURCE_CONTROL_USERNAME = os.getlogin()
SOURCE_CONTROL_WORKSPACE = 'fill_in_a_workspace_name'
SOURCE_CONTROL_TEST_PATH = "//UE4/Dev-VirtualProduction/Collaboration/VirtualProdTest/..."
SOURCE_CONTROL_SYNC_REVISION = "13658760"

class TestSwitchboardListener(unittest.TestCase):
    def setUp(self):
        self.socket = socket.socket(socket.AF_INET, socket.SOCK_STREAM)
        address = (LISTENER_IP, LISTENER_PORT)
        self.socket.connect(address)

    def tearDown(self):
        self.socket.shutdown(socket.SHUT_RDWR)
        self.socket.close()

    def test_command_accepted(self):
        run_cmd = {
                    "command": "start",
                    "id": str(uuid.uuid4()),
                    "exe": TEST_PROGRAM_EXE,
                    "args": ""
                }
        message = json.dumps(run_cmd).encode() + b'\x00'
        self.socket.send(message)

        received_data = self.socket.recv(10000)
        chunks = str(received_data.decode()).split('\x00')
        messages = [json.loads(chunk) for chunk in chunks if len(chunk) > 0]
        self.assertTrue(messages[0]["command accepted"])

    def test_command_denied(self):
        run_cmd = {
                    "command": "start",
                    "id": str(uuid.uuid4()),
                    "exe": TEST_PROGRAM_EXE,
                    "args": ""
                }
        message = json.dumps(run_cmd).encode() + b'\x00'
        self.socket.send(message)

        received_data = self.socket.recv(10000)
        chunks = str(received_data.decode()).split('\x00')
        messages = [json.loads(chunk) for chunk in chunks if len(chunk) > 0]
        self.assertTrue(messages[0]["command accepted"])

    def test_malformed_json(self):
        # missing closing brace
        run_cmd = "{ 'command': 'start', 'id': '16fd2706-8baf-433b-82eb-8c7fada847da', 'exe': 'C:\\Program Files\\Git\\bin\\git.exe, 'args': ''"
        message = run_cmd.encode() + b'\x00'
        self.socket.send(message)

        received_data = self.socket.recv(10000)
        chunks = str(received_data.decode()).split('\x00')
        messages = [json.loads(chunk) for chunk in chunks if len(chunk) > 0]
        self.assertFalse(messages[0]["command accepted"])

    def test_process_start_end(self):
        run_cmd = {
                    "command": "start",
                    "id": str(uuid.uuid4()),
                    "exe": TEST_PROGRAM_EXE,
                    "args": ""
                }
        message = json.dumps(run_cmd).encode() + b'\x00'
        self.socket.send(message)

        wait_for_ack = True
        wait_for_program_started = True
        wait_for_program_ended = True
        while wait_for_ack or wait_for_program_started or wait_for_program_ended:
            received_data = self.socket.recv(10000)
            chunks = str(received_data.decode()).split('\x00')
            messages = [json.loads(chunk) for chunk in chunks if len(chunk) > 0]
            for msg in messages:
                if "command accepted" in msg:
                    self.assertTrue(msg["command accepted"])
                    wait_for_ack = False
                elif "program started" in msg:
                    self.assertTrue(msg["program started"])
                    wait_for_program_started = False
                elif "program ended" in msg:
                    self.assertTrue(msg["program ended"])
                    self.assertEqual(msg["returncode"], 1)
                    wait_for_program_ended = False

    def test_program_start_failure(self):
        run_cmd = {
                    "command": "start",
                    "id": str(uuid.uuid4()),
                    "exe": "C:\\foo.exe",
                    "args": ""
                }
        message = json.dumps(run_cmd).encode() + b'\x00'
        self.socket.send(message)

        wait_for_ack = True
        wait_for_program_started = True
        while wait_for_ack or wait_for_program_started:
            received_data = self.socket.recv(10000)
            chunks = str(received_data.decode()).split('\x00')
            messages = [json.loads(chunk) for chunk in chunks if len(chunk) > 0]
            for msg in messages:
                if "command accepted" in msg:
                    self.assertTrue(msg["command accepted"])
                    wait_for_ack = False
                elif "program started" in msg:
                    self.assertFalse(msg["program started"])
                    wait_for_program_started = False

    @unittest.skip("Make sure to check the global variables at the top before running these")
    def test_vcs_init(self):
        vcs_cmd = {
            "command": "vcs init",
            "id": str(uuid.uuid4()),
            "provider": "perforce",
            "vcs settings": {
                'Port': SOURCE_CONTROL_CONNECTION,
                'UserName': SOURCE_CONTROL_USERNAME,
                'Workspace': SOURCE_CONTROL_WORKSPACE
                }
        }
        message = json.dumps(vcs_cmd).encode() + b'\x00'
        self.socket.send(message)

        wait_for_ack = True
        wait_for_init_completed = True
        while wait_for_ack or wait_for_init_completed:
            received_data = self.socket.recv(10000)
            chunks = str(received_data.decode()).split('\x00')
            messages = [json.loads(chunk) for chunk in chunks if len(chunk) > 0]
            for msg in messages:
                if "command accepted" in msg:
                    self.assertTrue(msg["command accepted"])
                    wait_for_ack = False
                elif "vcs init complete" in msg:
                    self.assertTrue(msg["vcs init complete"])
                    wait_for_init_completed = False

    def test_transfer_file_back_and_forth(self):
        with open(TEST_FILE_TO_TRANSFER, 'rb') as f:
            file_content = f.read()
        encoded_content = base64.b64encode(file_content)
        destination = "%TEMP%/sb_test/%RANDOM%.cfg"
        cmd = { "command": "send file", "id": str(uuid.uuid4()), "destination": destination, "content": encoded_content.decode() }
        message = json.dumps(cmd).encode() + b'\x00'
        self.socket.send(message)

        remote_path = ""

        wait_for_ack = True
        wait_for_send_completed = True
        while wait_for_ack or wait_for_send_completed:
            received_data = self.socket.recv(10000)
            chunks = str(received_data.decode()).split('\x00')
            messages = [json.loads(chunk) for chunk in chunks if len(chunk) > 0]
            for msg in messages:
                if "command accepted" in msg:
                    self.assertTrue(msg["command accepted"])
                    wait_for_ack = False
                elif "send file complete" in msg:
                    self.assertTrue(msg["send file complete"])
                    wait_for_send_completed = False
                    remote_path = msg["destination"]
                    self.assertTrue(os.path.exists(remote_path))

        receive_cmd = { "command": "receive file", "id": str(uuid.uuid4()), "source": remote_path }
        message = json.dumps(receive_cmd).encode() + b'\x00'
        self.socket.send(message)

        wait_for_ack = True
        wait_for_receive_completed = True
        while wait_for_ack or wait_for_receive_completed:
            received_data = self.socket.recv(30000)
            chunks = str(received_data.decode()).split('\x00')
            messages = [json.loads(chunk) for chunk in chunks if len(chunk) > 0]
            for msg in messages:
                if "command accepted" in msg:
                    self.assertTrue(msg["command accepted"])
                    wait_for_ack = False
                elif "receive file complete" in msg:
                    self.assertTrue(msg["receive file complete"])
                    wait_for_receive_completed = False
                    self.assertEqual(encoded_content.decode(), msg["content"])

        os.unlink(remote_path)


@unittest.skip("Make sure to check the global variables at the top before running these")
class TestSwitchboardListenerVersionControl(unittest.TestCase):
    def setUp(self):
        self.socket = socket.socket(socket.AF_INET, socket.SOCK_STREAM)
        address = (LISTENER_IP, LISTENER_PORT)
        self.socket.connect(address)

        init_cmd = {
            "command": "vcs init",
            "id": str(uuid.uuid4()),
            "provider": "perforce",
            "vcs settings": {
                'Port': SOURCE_CONTROL_CONNECTION,
                'UserName': SOURCE_CONTROL_USERNAME,
                'Workspace': SOURCE_CONTROL_WORKSPACE
                }
        }
        message = json.dumps(init_cmd).encode() + b'\x00'
        self.socket.send(message)

        wait_for_ack = True
        wait_for_init_completed = True
        while wait_for_ack or wait_for_init_completed:
            received_data = self.socket.recv(10000)
            chunks = str(received_data.decode()).split('\x00')
            messages = [json.loads(chunk) for chunk in chunks if len(chunk) > 0]
            for msg in messages:
                if "command accepted" in msg:
                    wait_for_ack = False
                elif "vcs init complete" in msg:
                    wait_for_init_completed = False

    def tearDown(self):
        self.socket.shutdown(socket.SHUT_RDWR)
        self.socket.close()

    def test_vcs_report_revision(self):
        vcs_cmd = {
            "command": "vcs report revision",
            "path": SOURCE_CONTROL_TEST_PATH,
            "id": str(uuid.uuid4())
        }
        message = json.dumps(vcs_cmd).encode() + b'\x00'
        self.socket.send(message)

        wait_for_ack = True
        wait_for_revision_report_completed = True
        while wait_for_ack or wait_for_revision_report_completed:
            received_data = self.socket.recv(10000)
            chunks = str(received_data.decode()).split('\x00')
            messages = [json.loads(chunk) for chunk in chunks if len(chunk) > 0]
            for msg in messages:
                if "command accepted" in msg:
                    self.assertTrue(msg["command accepted"])
                    wait_for_ack = False
                elif "vcs report revision complete" in msg:
                    self.assertTrue(msg["vcs report revision complete"])
                    self.assertTrue(msg["revision"].isnumeric())
                    wait_for_revision_report_completed = False

    def test_vcs_sync(self):
        vcs_sync_cmd = {
            "command": "vcs sync",
            "id": str(uuid.uuid4()),
            "revision": SOURCE_CONTROL_SYNC_REVISION,
            "path": SOURCE_CONTROL_TEST_PATH
        }
        message = json.dumps(vcs_sync_cmd).encode() + b'\x00'
        self.socket.send(message)

        wait_for_ack = True
        wait_for_sync_completed = True
        while wait_for_ack or wait_for_sync_completed:
            received_data = self.socket.recv(10000)
            chunks = str(received_data.decode()).split('\x00')
            messages = [json.loads(chunk) for chunk in chunks if len(chunk) > 0]
            for msg in messages:
                if "command accepted" in msg:
                    self.assertTrue(msg["command accepted"])
                    wait_for_ack = False
                elif "vcs sync complete" in msg:
                    self.assertTrue(msg["vcs sync complete"])
                    self.assertEqual(msg["revision"], SOURCE_CONTROL_SYNC_REVISION)
                    self.assertTrue(int(msg["revision"]) > 0)
                    wait_for_sync_completed = False

if __name__ == "__main__":
    unittest.main()