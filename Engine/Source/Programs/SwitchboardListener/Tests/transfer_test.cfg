#####################################################################
# nDisplay demo config file
#
# This file demonstrates how to configure a 5 sided CAVE. The CAVE's
# area is 3x3 meters. It consist of 3 walls, floor and ceiling screens.
# The height is 2 meters.
#
# Note:
# Before use, make sure all settings correspond to your system.
#####################################################################


#####################################################################
# Config info
#********************************************************************
# Config file header.
#
# Properties:
# version - Specifies the version of the configuration file (UE4.xx)
#********************************************************************
[info] version="23"


#####################################################################
# Cluster nodes
#********************************************************************
# Cluster node is an application instance. It's allowed to use
# multiple instances on the same PC. Sometimes its necessary. Usually
# application instances run on separate machines.
#
# Properties:
# id       - Unique node name
# addr     - Network address (IPv4 only)
# window   - Window ID
#
# Optional properties:
# master   - Specifies if current node is master; default is 'false'
# port_cs  - Cluster Synchronization port (used by master node only); default is 41001.
# port_ss  - Swap Synchronization port (used by master node only); default is 41002.
# port_ce  - Cluster Events port (used by master node only); default is 41003.
# sound    - turns on/off sound for this application instance; default is 'false'
#********************************************************************
[cluster_node] id="node_left"    addr="127.0.0.1" window="wnd_left"  master="true" sound="true"
[cluster_node] id="node_right"   addr="127.0.0.1" window="wnd_right"
#[cluster_node] id="node_botleft"   addr="127.0.0.1" window="wnd_botleft"
#[cluster_node] id="node_botright"   addr="127.0.0.1" window="wnd_botright"

#####################################################################
# Application windows
#********************************************************************
# The window entity defines properties of the application's game window.
#
# Properties:
# id         - Unique window name
# viewports  - Array of viewports
#
# Optional properties:
# fullscreen - Fullscreen or windowed mode; default is false - windowed
# winx       - X location
# winy       - Y location
# resx       - Width
# resy       - Height
#********************************************************************
[window] id="wnd_left"    viewports="vp_left" winx=2560 winy=0 resx="480" resy="320" fullscreen=false
[window] id="wnd_right"   viewports="vp_right" winx=3040 winy=0 resx="480" resy="320" fullscreen=false
#[window] id="wnd_botleft"    viewports="vp_botleft" winx=2560 winy=320 resx="480" resy="320" fullscreen=false
#[window] id="wnd_botright"   viewports="vp_botright" winx=3040 winy=320 resx="480" resy="320" fullscreen=false

#####################################################################
# Viewports
#********************************************************************
# Viewport is a rectangle area of game window where a frame is mapped.
# Usually viewports start at 0:0 and have the same size as its parent
# window (single viewport rendering). However, in some cases multiple
# viewports required.
#
# Properties:
# id     - Unique viewport name
# x      - X coordinate of viewport's top left corner on a window
# y      - Y coordinate of viewport's top left corner on a window
# width  - Width of viewport in pixels
# height - Height of viewport in pixels
#
# Optional properties:
# camera - Cammera assigned to this viewport
#********************************************************************
[viewport] id="vp_left"    x="0" y="0" width="480" height="320" projection="proj_simple_left"
[viewport] id="vp_right"   x="0" y="0" width="480" height="320" projection="proj_simple_right"
#[viewport] id="vp_botleft"    x="0" y="0" width="480" height="320" projection="proj_simple_botleft"
#[viewport] id="vp_botright"   x="0" y="0" width="480" height="320" projection="proj_simple_botright"


#####################################################################
# Projection policies
#********************************************************************
# Projection policy is an abstraction which knows where to take an
# projection input data and how to compute the output. This means
# that each policy might have it's own properties that it knows how
# to interpret and utilize. There are several projection policies
# available natively: simple, MPCDI and EasyBlend:
#
# * Simple  - This one is the native projection policy that can be fully
#             configured from the config file. All you need is to specify
#             a projection screen. Those who has been used the nDisplay
#             before are familiar with this policy already.
#
# * MPCDI   - This policy allows to use an industry standard for multi-
#             projector setups. It requires a special .mpcdi file that
#             contains all the data required for computations.
#
# * EasyBlend - This policy is an implementation of the EasyBLend SDK
#               provided by Scalable Display Technologies. It requires
#               a bunch of special files that describe a multi-projector
#               system.
#
# Properties:
# id     - Unique projection policy name
# type   - Projection policy type (custom types can be used)
#********************************************************************
# Here is an example of simple projection policy. It requires to
# specify projection screens only. Refer to specific demo configs
# or documentation page to get more info about other projection policies.
[projection] id="proj_simple_left"    type="simple" screen="scr_left"
[projection] id="proj_simple_right"   type="simple" screen="scr_right"
#[projection] id="proj_simple_botleft"    type="simple" screen="scr_botleft"
#[projection] id="proj_simple_botright"   type="simple" screen="scr_botright"

#####################################################################
# Projection screens
#********************************************************************
# Projection screen is a rectangle which determines the camera frustum.
# Usually the projection screen has the same dimensions as an output
# display but in some cases it may differ. Used by simple projection
# policy.
#
# Properties:
# id     - unique projection screen name
# loc    - relative location to the parent component. Location is relative
#          to the VR root if no parent specified. The pivot is a screen's
#          center and the values are in meters.
# rot    - relative rotation to the parent component. Rotation is relative
#          to the VR root if no parent specified. The pivot is a screen's
#          center and the values are in degrees.
# size   - width (X) and height (Y) of the screen. Values are in meters.
#
# Optional properties:
# parent     - ID of parent component in VR hierarchy; default is VR root.
# tracker_id - ID of tracking device; no tracking by default.
# tracker_ch - ID of tracking device's channel; no tracking by default.
#********************************************************************
[screen] id="scr_left"    loc="X=0,Y=0,Z=0" rot="P=0,Y=0,R=0" size="X=3,Y=2"   parent="display_left"
[screen] id="scr_right"   loc="X=0,Y=0,Z=0" rot="P=0,Y=0,R=0" size="X=3,Y=2"   parent="display_right"
#[screen] id="scr_botleft"   loc="X=0,Y=0,Z=0" rot="P=0,Y=0,R=0" size="X=3,Y=2"   parent="display_botleft"
#[screen] id="scr_botright"   loc="X=0,Y=0,Z=0" rot="P=0,Y=0,R=0" size="X=3,Y=2"   parent="display_botright"



#####################################################################
# Cameras
#********************************************************************
# Camera is a predefined point frome where the stereoscopic view built.
# It's possible to define multiple cameras and swith the active one
# during runtime. You're free to attach any camera to a tracking device
# for head tracking. Consider a camera as a viewer's head.
#
# Properties:
# id  - unique camera name
# loc - relative location to the parent component. Location is relative
#       to the VR root if no parent specified.
# rot - relative rotation to the parent component. Rotation is relative
#       to the VR root if no parent specified.
#
# Optional properties:
# parent       - ID of parent component in VR hierarchy; default is VR root.
# tracker_id   - ID of tracking device; no tracking by default.
# tracker_ch   - ID of tracking device's channel; no tracking by default.
# eye_swap     - Swap eyes for this camera in stereo mode; default is 'false'
# eye_dist     - interoccular distance in meters for stereo/force_offset modes.
#                Default is 0.064.
# force_offset - Force monoscopic camera to behave like a left/right eye
#                of a stereoscopic camera. Eye_offset is applicable in this case.
#********************************************************************
[camera] id="camera_static" loc="X=-1,Y=0,Z=0" parent="socket_cam" eye_swap="false" eye_dist="0.064" force_offset="0"

#####################################################################
# Scene nodes (hierarchy transforms)
#********************************************************************
# Scene node is an actor component which is basically a transformation
# matrix. Scene nodes can be helpful to build a component hierarchy, to
# define some special places (like a socket) within VR space.
#
# It might be difficult to understand what VR space origin is. Consider
# it as a point in space where VR space starts. Any componenent listed
# in this config file is relative to its parent or this origin.
#
# Properties:
# id  - unique scene node name
# loc - relative location to the parent component. Location is relative
#       to the VR root if no parent specified.
# rot - relative rotation to the parent component. Rotation is relative
#       to the VR root if no parent specified.
#
# Optional properties:
# parent     - ID of parent component in VR hierarchy; default is VR root.
# tracker_id - ID of tracking device; no tracking by default.
# tracker_ch - ID of tracking device's channel; no tracking by default.
#********************************************************************
[scene_node] id="cave_origin"     loc="X=0,Y=0,Z=0"      rot="P=0,Y=0,R=0"

[scene_node] id="cave_center"     loc="X=0,Y=0,Z=1"      rot="P=0,Y=0,R=0"   parent="cave_origin"
[scene_node] id="display_left"    loc="X=0,Y=-1.5,Z=1"   rot="P=0,Y=0,R=0"   parent="cave_center"
[scene_node] id="display_right"   loc="X=0,Y=1.5,Z=1"    rot="P=0,Y=0,R=0"   parent="cave_center"
#[scene_node] id="display_botleft"    loc="X=0,Y=-1.5,Z=-1"   rot="P=0,Y=0,R=0"   parent="cave_center"
#[scene_node] id="display_botright"   loc="X=0,Y=1.5,Z=-1"    rot="P=0,Y=0,R=0"   parent="cave_center"


[scene_node] id="socket_cam"      loc="X=0,Y=0,Z=1.7"    rot="P=0,Y=0,R=0"   parent="cave_origin"
[scene_node] id="wand"            loc="X=0,Y=0,Z=1"


#####################################################################
# Input devices
#********************************************************************
# Input devices are VRPN devices. The nDisplay supports the following
# types: analog, button and tracker. Many of physical input devices
# can be connected via VRPN.
#
# Properties:
# id    - nique device name
# type  - VRPN type (analog, button or tracker).
# addr  - address of a VRPN server which handles this particular device.
#         The value must match the following format: DEVICENAME@SERVER_ADDRESS
#         where DEVICENAME is a VRPN name of this device and SERVER_ADDRESS
#         is IPv4 address of VRPN server.
# loc   - relative location to the parent component. Location is relative
#         to the VR root if no parent specified.
# rot   - relative rotation to the parent component. Rotation is relative
#         to the VR root if no parent specified.
#
# front (tracker only) - mapping of a tracking system axis to X axis of VR origin
# right (tracker only) - mapping of a tracking system axis to Y axis of VR origin
# up    (tracker only) - mapping of a tracking system axis to Z axis of VR origin
# * The following values are allowed for axes mapping: X, -X, Y, -Y, Z, -Z
#
# Optional properties:
# remap - VRPN device channel remapping. Value format is: "from0:to0,from1:to1,...,fromN:toN".
#         For example: remap="0:3,1:4,5:2"
#********************************************************************
[input] id="MAxis"     type="analog"  addr=Mouse0@127.0.0.1
[input] id="MButtons"  type="buttons" addr=Mouse0@127.0.0.1
[input] id="KButtons"  type="buttons" addr=Keyboard0@127.0.0.1


#####################################################################
# General settings
#********************************************************************
# Properties:
# swap_sync_policy - swap synchronization policy
#                  - 0 - no synchronization
#                  - 1 - software swap synchronization
#                  - 2 - NV swap lock (Nvidia cards only, OpenGL only)
[general] swap_sync_policy="1"


#####################################################################
# Network settings
#********************************************************************
# Optional properties:
# cln_conn_tries_amount - how many times a client tries to connect to a server; default is 10
# cln_conn_retry_delay  - delay before next client connection try (milliseconds); default is 1000
# game_start_timeout    - timeout before all data is loaded and game started (milliseconds); default is 30000
# barrier_wait_timeout  - barrier timeout for both game and render threads (milliseconds); default is 5000
[network] cln_conn_tries_amount="10" cln_conn_retry_delay="1000" game_start_timeout="300000" barrier_wait_timeout="500000"


#####################################################################
# Custom arguments
#********************************************************************
# Any custom arguments available in runtime can be specified here.
# Format:  ARG_NAME="ARG_VAL"
[custom] SampleArg1="SampleVal1" SampleArg2="SampleVal2"
